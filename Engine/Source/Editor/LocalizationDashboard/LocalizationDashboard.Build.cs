// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class LocalizationDashboard : ModuleRules
{
	public LocalizationDashboard(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
                "CoreUObject",
                "PropertyEditor",
                "Localization"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"Slate",
				"SlateCore",
				"InputCore",
				"EditorFramework",
                "UnrealEd",
				"DesktopPlatform",
                "TranslationEditor",
                "MainFrame",
                "SourceControl",
                "SharedSettingsWidgets",
				"LocalizationCommandletExecution",
				"LocalizationService",
				"InternationalizationSettings",
				"ToolMenus",
				"WorkspaceMenuStructure",
			}
		);

        CircularlyReferencedDependentModules.AddRange(
           new string[] {
                "LocalizationService",
				"MainFrame",
				"TranslationEditor"
            }
           );
	}
}
