// Copyright Epic Games, Inc. All Rights Reserved.

#include "GatherTextDetailCustomizations.h"

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "DesktopPlatformModule.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericWindow.h"
#include "HAL/PlatformCrt.h"
#include "IDesktopPlatform.h"
#include "IDetailChildrenBuilder.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "LocalizationTargetTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Object.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SErrorHint.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class SWidget;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "GatherTextDetailCustomizations"

namespace
{
	class SGatherTextConfigurationErrorHint : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SGatherTextConfigurationErrorHint) {}
			SLATE_ATTRIBUTE(FText, ErrorText)
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& Arguments);
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	private:
		TAttribute<FText> ErrorText;
		TSharedPtr<SErrorHint> ErrorHint;
	};

	void SGatherTextConfigurationErrorHint::Construct(const FArguments& Arguments)
	{
		ErrorText = Arguments._ErrorText;

		ChildSlot
			[
				SAssignNew(ErrorHint, SErrorHint)
			];
	}

	void SGatherTextConfigurationErrorHint::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		if (ErrorText.IsSet() && ErrorHint.IsValid())
		{
			ErrorHint->SetError(ErrorText.Get());
		}
	}
}

namespace
{
	class SGatherTextPathPicker : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SGatherTextPathPicker)
			: _Font()
			, _ShouldCoercePathAsWildcardPattern(false)
		{}
			SLATE_ATTRIBUTE(FSlateFontInfo, Font)
			SLATE_ARGUMENT(bool, ShouldCoercePathAsWildcardPattern)
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& Args, const TSharedRef<IPropertyHandle>& InPathRootPropertyHandle, const TSharedRef<IPropertyHandle>& InPathStringPropertyHandle);

	private:
		FReply PathPickerOnClicked();
		ULocalizationTarget* GetLocalizationTarget() const;
		ELocalizationGatherPathRoot GetPathRoot() const;
		FString GetResolvedPathRoot() const;
		FText GetResolvedPathRootDisplayName() const;

	private:
		TSharedPtr<IPropertyHandle> PathRootPropertyHandle;
		TSharedPtr<IPropertyHandle> PathStringPropertyHandle;
		bool ShouldCoercePathAsWildcardPattern;
	};

	void SGatherTextPathPicker::Construct(const FArguments& Args, const TSharedRef<IPropertyHandle>& InPathRootPropertyHandle, const TSharedRef<IPropertyHandle>& InPathStringPropertyHandle)
	{
		PathRootPropertyHandle = InPathRootPropertyHandle;
		PathStringPropertyHandle = InPathStringPropertyHandle;
		ShouldCoercePathAsWildcardPattern = Args._ShouldCoercePathAsWildcardPattern;

		ChildSlot
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					PathRootPropertyHandle.IsValid() && PathRootPropertyHandle->IsValidHandle() ? PathRootPropertyHandle->CreatePropertyValueWidget() : SNullWidget::NullWidget
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					SNew(STextBlock)
					.Font(Args._Font)
					.Text(LOCTEXT("Colon", ":"))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					SNew(STextBlock)
					.Font(Args._Font)
					.Text(this, &SGatherTextPathPicker::GetResolvedPathRootDisplayName)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.MinDesiredWidth(125.0f)
					[
						PathStringPropertyHandle.IsValid() && PathStringPropertyHandle->IsValidHandle() ? PathStringPropertyHandle->CreatePropertyValueWidget() : SNullWidget::NullWidget
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
					.ToolTipText( LOCTEXT( "PatchPickerToolTipText", "Choose a directory.") )
					.OnClicked(this, &SGatherTextPathPicker::PathPickerOnClicked)
					.ContentPadding( 2.0f )
					.ForegroundColor( FSlateColor::UseForeground() )
					.IsFocusable( false )
					[
						SNew( SImage )
						.Image( FAppStyle::GetBrush("LocalizationTargetEditor.DirectoryPicker") )
						.ColorAndOpacity( FSlateColor::UseForeground() )
					]
				]
			];
	}

	FReply SGatherTextPathPicker::PathPickerOnClicked()
	{
		if (!PathStringPropertyHandle.IsValid() || !PathStringPropertyHandle->IsValidHandle())
		{
			return FReply::Handled();
		}

		IDesktopPlatform* const DesktopPlatform = FDesktopPlatformModule::Get();
		check(DesktopPlatform);

		const FString DesiredRootPath = FPaths::ConvertRelativePathToFull(GetResolvedPathRoot());

		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		const TSharedPtr<FGenericWindow> ParentGenericWindow = ParentWindow.IsValid() ? ParentWindow->GetNativeWindow() : nullptr;
		const void* ParentWindowHandle = ParentGenericWindow.IsValid() ? ParentGenericWindow->GetOSWindowHandle() : nullptr;
		const FText DialogTitle = LOCTEXT("SelectSearchDirectoryDialogTitle", "Select Directory Containing Text Files");

		FString CurrentPath;
		PathStringPropertyHandle->GetValue(CurrentPath);
		if (FPaths::IsRelative(CurrentPath))
		{
			CurrentPath = FPaths::Combine(*DesiredRootPath, *CurrentPath);
			int32 WildcardStarIndex;
			CurrentPath.FindLastChar(TEXT('*'), WildcardStarIndex);
			if (WildcardStarIndex != INDEX_NONE)
			{
				CurrentPath.LeftInline(WildcardStarIndex, EAllowShrinking::No);
			}
		}

		const FString DefaultPath = CurrentPath.IsEmpty() ? DesiredRootPath : CurrentPath;

		FString NewPath = DefaultPath;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowHandle, DialogTitle.ToString(), DefaultPath, NewPath))
		{
			FPaths::MakePathRelativeTo(NewPath, *DesiredRootPath);

			if (ShouldCoercePathAsWildcardPattern)
			{
				if (!NewPath.Contains(TEXT("*")))
				{
					NewPath = FPaths::Combine(*NewPath, TEXT("*"));
				}
			}
			PathStringPropertyHandle->SetValue(NewPath);
		}

		return FReply::Handled();
	}

	ULocalizationTarget* SGatherTextPathPicker::GetLocalizationTarget() const
	{
		if (!PathStringPropertyHandle.IsValid() || !PathStringPropertyHandle->IsValidHandle())
		{
			return nullptr;
		}

		TArray<UObject*> OuterObjects;
		PathStringPropertyHandle->GetOuterObjects(OuterObjects);
		return CastChecked<ULocalizationTarget>(OuterObjects.Top());
	}

	ELocalizationGatherPathRoot SGatherTextPathPicker::GetPathRoot() const
	{
		if (!PathRootPropertyHandle.IsValid() || !PathRootPropertyHandle->IsValidHandle())
		{
			return ELocalizationGatherPathRoot::Auto;
		}

		TArray<const void*> RawData;
		PathRootPropertyHandle->AccessRawData(RawData);
		return RawData.Top()
			? *(ELocalizationGatherPathRoot*)RawData.Top()
			: ELocalizationGatherPathRoot::Auto;
	}

	FString SGatherTextPathPicker::GetResolvedPathRoot() const
	{
		ULocalizationTarget* LocalizationTarget = GetLocalizationTarget();
		return FLocalizationGatherPathRootUtil::GetResolvedPathRoot(GetPathRoot(), LocalizationTarget && LocalizationTarget->IsMemberOfEngineTargetSet());
	}

	FText SGatherTextPathPicker::GetResolvedPathRootDisplayName() const
	{
		ULocalizationTarget* LocalizationTarget = GetLocalizationTarget();
		return FLocalizationGatherPathRootUtil::GetResolvedPathRootDisplayName(GetPathRoot(), LocalizationTarget && LocalizationTarget->IsMemberOfEngineTargetSet());
	}
}

void FGatherTextSearchDirectoryStructCustomization::CustomizeStructHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils )
{
	const TSharedPtr<IPropertyHandle> PathRootPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGatherTextSearchDirectory, PathRoot));
	const TSharedPtr<IPropertyHandle> PathStringPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGatherTextSearchDirectory, Path));

	const auto& GetErrorText = [StructPropertyHandle]() -> FText
	{
		if (StructPropertyHandle->IsValidHandle())
		{
			TArray<const void*> RawData;
			StructPropertyHandle->AccessRawData(RawData);
			if (const FGatherTextSearchDirectory* Struct = (reinterpret_cast<const FGatherTextSearchDirectory*>(RawData.Top())))
			{
				TArray<UObject*> OuterObjects;
				StructPropertyHandle->GetOuterObjects(OuterObjects);
				ULocalizationTarget* const LocalizationTarget = Cast<ULocalizationTarget>(OuterObjects.Top());
				if (LocalizationTarget)
				{
					FText Error;
					if (!Struct->Validate(LocalizationTarget->IsMemberOfEngineTargetSet(), Error))
					{
						return Error;
					}
				}
			}
		}

		return FText::GetEmpty();
	};

	HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SGatherTextConfigurationErrorHint)
				.ErrorText_Lambda(GetErrorText)
			]
		]
		.ValueContent()
		.MaxDesiredWidth(TOptional<float>())
		[
			SNew(SGatherTextPathPicker, PathRootPropertyHandle.ToSharedRef(), PathStringPropertyHandle.ToSharedRef())
			.Font(StructCustomizationUtils.GetRegularFont())
		];
}

void FGatherTextIncludePathStructCustomization::CustomizeStructHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils )
{
	const TSharedPtr<IPropertyHandle> PathRootPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGatherTextIncludePath, PathRoot));
	const TSharedPtr<IPropertyHandle> PatternPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGatherTextIncludePath, Pattern));

	const auto& GetErrorText = [StructPropertyHandle]() -> FText
	{
		if (StructPropertyHandle->IsValidHandle())
		{
			TArray<const void*> RawData;
			StructPropertyHandle->AccessRawData(RawData);
			const FGatherTextIncludePath& Struct = *(reinterpret_cast<const FGatherTextIncludePath*>(RawData.Top()));

			TArray<UObject*> OuterObjects;
			StructPropertyHandle->GetOuterObjects(OuterObjects);
			ULocalizationTarget* const LocalizationTarget = Cast<ULocalizationTarget>(OuterObjects.Top());
			if (LocalizationTarget)
			{
				FText Error;
				if (!Struct.Validate(LocalizationTarget->IsMemberOfEngineTargetSet(), Error))
				{
					return Error;
				}
			}
		}

		return FText::GetEmpty();
	};

	HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SGatherTextConfigurationErrorHint)
				.ErrorText_Lambda(GetErrorText)
			]
		]
		.ValueContent()
		.MaxDesiredWidth(TOptional<float>())
		[
			SNew(SGatherTextPathPicker, PathRootPropertyHandle.ToSharedRef(), PatternPropertyHandle.ToSharedRef())
			.Font(StructCustomizationUtils.GetRegularFont())
			.ShouldCoercePathAsWildcardPattern(true)
		];
}

void FGatherTextExcludePathStructCustomization::CustomizeStructHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils )
{
	const TSharedPtr<IPropertyHandle> PathRootPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGatherTextExcludePath, PathRoot));
	const TSharedPtr<IPropertyHandle> PatternPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGatherTextExcludePath, Pattern));

	const auto& GetErrorText = [StructPropertyHandle]() -> FText
	{
		if (StructPropertyHandle->IsValidHandle())
		{
			TArray<const void*> RawData;
			StructPropertyHandle->AccessRawData(RawData);
			const FGatherTextExcludePath& Struct = *(reinterpret_cast<const FGatherTextExcludePath*>(RawData.Top()));

			TArray<UObject*> OuterObjects;
			StructPropertyHandle->GetOuterObjects(OuterObjects);
			ULocalizationTarget* const LocalizationTarget = Cast<ULocalizationTarget>(OuterObjects.Top());
			if (LocalizationTarget)
			{
				FText Error;
				if (!Struct.Validate(LocalizationTarget->IsMemberOfEngineTargetSet(), Error))
				{
					return Error;
				}
			}
		}

		return FText::GetEmpty();
	};

	HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SGatherTextConfigurationErrorHint)
				.ErrorText_Lambda(GetErrorText)
			]
		]
		.ValueContent()
		.MaxDesiredWidth(TOptional<float>())
		[
			SNew(SGatherTextPathPicker, PathRootPropertyHandle.ToSharedRef(), PatternPropertyHandle.ToSharedRef())
			.Font(StructCustomizationUtils.GetRegularFont())
			.ShouldCoercePathAsWildcardPattern(true)
		];
}

void FGatherTextFileExtensionStructCustomization::CustomizeStructHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils )
{
	const TSharedPtr<IPropertyHandle> PatternPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGatherTextFileExtension, Pattern));

	const auto& GetErrorText = [StructPropertyHandle]() -> FText
	{
		if (StructPropertyHandle->IsValidHandle())
		{
			TArray<const void*> RawData;
			StructPropertyHandle->AccessRawData(RawData);
			const FGatherTextFileExtension& Struct = *(reinterpret_cast<const FGatherTextFileExtension*>(RawData.Top()));

			TArray<UObject*> OuterObjects;
			StructPropertyHandle->GetOuterObjects(OuterObjects);
			ULocalizationTarget* const LocalizationTarget = Cast<ULocalizationTarget>(OuterObjects.Top());
			if (LocalizationTarget)
			{
				FText Error;
				if (!Struct.Validate(LocalizationTarget->IsMemberOfEngineTargetSet(), Error))
				{
					return Error;
				}
			}
		}

		return FText::GetEmpty();
	};

	HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SGatherTextConfigurationErrorHint)
				.ErrorText_Lambda(GetErrorText)
			]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(TEXT("*.")))
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				PatternPropertyHandle.IsValid() && PatternPropertyHandle->IsValidHandle() ? PatternPropertyHandle->CreatePropertyValueWidget() : SNullWidget::NullWidget
			]
		];
}

namespace
{
	class SConfigurationValidity : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SConfigurationValidity) {}
			SLATE_ATTRIBUTE(FText, ConfigurationError)
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& Arguments);

	private:
		const FSlateBrush* GetImageBrush() const;

	private:
		TAttribute<FText> ConfigurationError;
	};

	void SConfigurationValidity::Construct(const FArguments& Arguments)
	{
		ConfigurationError = Arguments._ConfigurationError;

		const auto& GetToolTipText = [this]() -> FText
		{
			FText ToolTipText = ConfigurationError.Get(FText::GetEmpty());
			if (ToolTipText.IsEmpty())
			{
				ToolTipText = LOCTEXT("ValidGatherConfigurationToolTip", "Configuration settings are valid.");
			}
			return ToolTipText;
		};

		ChildSlot
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(this, &SConfigurationValidity::GetImageBrush)
				.ToolTipText_Lambda(GetToolTipText)
			];
	}

	const FSlateBrush* SConfigurationValidity::GetImageBrush() const
	{
		return ConfigurationError.Get(FText::GetEmpty()).IsEmpty() ? FAppStyle::GetBrush("LocalizationTargetEditor.GatherSettingsIcon_Valid") : FAppStyle::GetBrush("LocalizationTargetEditor.GatherSettingsIcon_Warning");
	}
}

void FGatherTextFromTextFilesConfigurationStructCustomization::CustomizeStructHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils )
{
	const TSharedPtr<IPropertyHandle> IsEnabledPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGatherTextFromTextFilesConfiguration, IsEnabled));
	if (IsEnabledPropertyHandle.IsValid() && IsEnabledPropertyHandle->IsValidHandle())
	{
		IsEnabledPropertyHandle->MarkHiddenByCustomization();
	}

	const auto& GetConfigurationError = [StructPropertyHandle]() -> FText
	{
		TArray<const void*> RawData;
		StructPropertyHandle->AccessRawData(RawData);
		const FGatherTextFromTextFilesConfiguration& Struct = *(reinterpret_cast<const FGatherTextFromTextFilesConfiguration*>(RawData.Top()));
		
		TArray<UObject*> OuterObjects;
		StructPropertyHandle->GetOuterObjects(OuterObjects);
		ULocalizationTarget* const LocalizationTarget = Cast<ULocalizationTarget>(OuterObjects.Top());
		if (LocalizationTarget)
		{
			FText Error;
			if (!Struct.Validate(LocalizationTarget->IsMemberOfEngineTargetSet(), Error))
			{
				return Error;
			}
		}

		return FText::GetEmpty();
	};

	const auto& GetValidityWidgetVisibility = [IsEnabledPropertyHandle]() -> EVisibility
	{
		bool Value = false;
		if (IsEnabledPropertyHandle.IsValid() && IsEnabledPropertyHandle->IsValidHandle())
		{
			IsEnabledPropertyHandle->GetValue(Value);
		}
		return Value ? EVisibility::Visible : EVisibility::Hidden;
	};
		
	HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				IsEnabledPropertyHandle.IsValid() && IsEnabledPropertyHandle->IsValidHandle() ? IsEnabledPropertyHandle->CreatePropertyValueWidget() : SNullWidget::NullWidget
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				StructPropertyHandle->CreatePropertyValueWidget()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SConfigurationValidity)
				.Visibility_Lambda(GetValidityWidgetVisibility)
				.ConfigurationError_Lambda(GetConfigurationError)
			]
		];
}

void FGatherTextFromTextFilesConfigurationStructCustomization::CustomizeStructChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils )
{
	uint32 ChildCount;
	StructPropertyHandle->GetNumChildren(ChildCount);

	for (uint32 i = 0; i < ChildCount; ++i)
	{
		const TSharedPtr<IPropertyHandle> ChildPropertyHandle = StructPropertyHandle->GetChildHandle(i);
		if (!ChildPropertyHandle->IsCustomized())
		{
			ChildBuilder.AddProperty(ChildPropertyHandle.ToSharedRef());
		}
	}
}

void FGatherTextFromPackagesConfigurationStructCustomization::CustomizeStructHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils )
{
	const TSharedPtr<IPropertyHandle> IsEnabledPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGatherTextFromPackagesConfiguration, IsEnabled));
	if (IsEnabledPropertyHandle.IsValid() && IsEnabledPropertyHandle->IsValidHandle())
	{
		IsEnabledPropertyHandle->MarkHiddenByCustomization();
	}

	const auto& GetConfigurationError = [StructPropertyHandle]() -> FText
	{
		TArray<const void*> RawData;
		StructPropertyHandle->AccessRawData(RawData);
		const FGatherTextFromPackagesConfiguration& Struct = *(reinterpret_cast<const FGatherTextFromPackagesConfiguration*>(RawData.Top()));

		TArray<UObject*> OuterObjects;
		StructPropertyHandle->GetOuterObjects(OuterObjects);
		ULocalizationTarget* const LocalizationTarget = Cast<ULocalizationTarget>(OuterObjects.Top());
		if (LocalizationTarget)
		{
			FText Error;
			if (!Struct.Validate(LocalizationTarget->IsMemberOfEngineTargetSet(), Error))
			{
				return Error;
			}
		}

		return FText::GetEmpty();
	};

	const auto& GetValidityWidgetVisibility = [IsEnabledPropertyHandle]() -> EVisibility
	{
		bool Value = false;
		if (IsEnabledPropertyHandle.IsValid() && IsEnabledPropertyHandle->IsValidHandle())
		{
			IsEnabledPropertyHandle->GetValue(Value);
		}
		return Value ? EVisibility::Visible : EVisibility::Hidden;
	};
		
	HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				IsEnabledPropertyHandle.IsValid() && IsEnabledPropertyHandle->IsValidHandle() ? IsEnabledPropertyHandle->CreatePropertyValueWidget() : SNullWidget::NullWidget
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				StructPropertyHandle->CreatePropertyValueWidget()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SConfigurationValidity)
				.Visibility_Lambda(GetValidityWidgetVisibility)
				.ConfigurationError_Lambda(GetConfigurationError)
			]
		];
}

void FGatherTextFromPackagesConfigurationStructCustomization::CustomizeStructChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils )
{
	uint32 ChildCount;
	StructPropertyHandle->GetNumChildren(ChildCount);

	for (uint32 i = 0; i < ChildCount; ++i)
	{
		const TSharedPtr<IPropertyHandle> ChildPropertyHandle = StructPropertyHandle->GetChildHandle(i);
		if (!ChildPropertyHandle->IsCustomized())
		{
			ChildBuilder.AddProperty(ChildPropertyHandle.ToSharedRef());
		}
	}
}

namespace
{
	class SMetaDataTextKeyPatternWidget : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SMetaDataTextKeyPatternWidget) {}
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& Args, const TSharedRef<IPropertyHandle>& PatternPropertyHandle);

	private:
		FText GetText() const;
		void OnTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);
		void OnPlaceHolderSelected(TSharedPtr<FString> PlaceHolderString, ESelectInfo::Type SelectInfo);

	private:
		TSharedPtr<IPropertyHandle> PatternPropertyHandle;
		TSharedPtr<SEditableTextBox> EditableTextBox;
		TSharedPtr<SComboButton> PlaceHolderComboButton;
		TArray< TSharedPtr<FString> > PossiblePlaceHoldersListItemsSource;
	};

	void SMetaDataTextKeyPatternWidget::Construct(const FArguments& Args, const TSharedRef<IPropertyHandle>& InPatternPropertyHandle)
	{
		PatternPropertyHandle = InPatternPropertyHandle;

		for(const FString& PossiblePlaceHolder : FMetaDataTextKeyPattern::GetPossiblePlaceHolders())
		{
			PossiblePlaceHoldersListItemsSource.Add( MakeShareable(new FString(PossiblePlaceHolder)) );
		}

		const auto& OnGenerateRow = [](TSharedPtr<FString> PlaceHolderString, const TSharedRef<STableViewBase>& Table) -> TSharedRef<ITableRow>
		{
			return SNew(STableRow< TSharedPtr<FString> >, Table)
				.Content()
				[
					SNew(SBox)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(FText::FromString(*PlaceHolderString.Get()))
					]
				];
		};

		ChildSlot
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.MinDesiredWidth(125.0f)
					[
						SAssignNew(EditableTextBox, SEditableTextBox)
						.Text(this, &SMetaDataTextKeyPatternWidget::GetText)
						.OnTextCommitted(this, &SMetaDataTextKeyPatternWidget::OnTextCommitted)
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					SAssignNew(PlaceHolderComboButton, SComboButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("LocalizationTargetEditor.GatherSettings.AddMetaDataTextKeyPatternArgument"))
					]
					.OnGetMenuContent_Lambda([&]() -> TSharedRef<SWidget>
					{
						return SNew(SListView< TSharedPtr<FString> >)
							.ListItemsSource(&PossiblePlaceHoldersListItemsSource)
							.OnGenerateRow_Lambda(OnGenerateRow)
							.OnSelectionChanged(this, &SMetaDataTextKeyPatternWidget::OnPlaceHolderSelected);
					})
				]
			];
	}

	FText SMetaDataTextKeyPatternWidget::GetText() const
	{
		if (!PatternPropertyHandle.IsValid() || !PatternPropertyHandle->IsValidHandle())
		{
			return FText::GetEmpty();
		}

		FString PatternString;
		PatternPropertyHandle->GetValue(PatternString);

		return FText::FromString(PatternString);
	}

	void SMetaDataTextKeyPatternWidget::OnTextCommitted( const FText& NewText, ETextCommit::Type CommitInfo )
	{
		if (!PatternPropertyHandle.IsValid() || !PatternPropertyHandle->IsValidHandle())
		{
			return;
		}

		const FString PatternString = NewText.ToString();
		PatternPropertyHandle->SetValue(PatternString);
	}

	void SMetaDataTextKeyPatternWidget::OnPlaceHolderSelected(TSharedPtr<FString> PlaceHolderString, ESelectInfo::Type SelectInfo)
	{
		if (PlaceHolderString.IsValid() && EditableTextBox.IsValid())
		{
			EditableTextBox->SetText(FText::FromString(EditableTextBox->GetText().ToString() + *PlaceHolderString));
			FSlateApplication::Get().SetKeyboardFocus(EditableTextBox);
			if(PlaceHolderComboButton.IsValid())
			{
				PlaceHolderComboButton->SetIsOpen(false);
			}
		}
	}
}

void FMetaDataTextKeyPatternStructCustomization::CustomizeStructHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils )
{
	const TSharedPtr<IPropertyHandle> PatternPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMetaDataTextKeyPattern, Pattern));

	const auto& GetErrorText = [StructPropertyHandle]() -> FText
	{
		if (StructPropertyHandle->IsValidHandle())
		{
			TArray<const void*> RawData;
			StructPropertyHandle->AccessRawData(RawData);
			const FMetaDataTextKeyPattern& Struct = *(reinterpret_cast<const FMetaDataTextKeyPattern*>(RawData.Top()));

			TArray<UObject*> OuterObjects;
			StructPropertyHandle->GetOuterObjects(OuterObjects);
			ULocalizationTarget* const LocalizationTarget = Cast<ULocalizationTarget>(OuterObjects.Top());
			if (LocalizationTarget)
			{
				FText Error;
				if (!Struct.Validate(LocalizationTarget->IsMemberOfEngineTargetSet(), Error))
				{
					return Error;
				}
			}
		}

		return FText::GetEmpty();
	};

	HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SGatherTextConfigurationErrorHint)
				.ErrorText_Lambda(GetErrorText)
			]
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SMetaDataTextKeyPatternWidget, PatternPropertyHandle.ToSharedRef())
		];
}

void FMetaDataKeyNameStructCustomization::CustomizeStructHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils )
{
	const TSharedPtr<IPropertyHandle> NamePropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMetaDataKeyName, Name));

	const auto& GetErrorText = [StructPropertyHandle]() -> FText
	{
		if (StructPropertyHandle->IsValidHandle())
		{
			TArray<const void*> RawData;
			StructPropertyHandle->AccessRawData(RawData);
			const FMetaDataKeyName& Struct = *(reinterpret_cast<const FMetaDataKeyName*>(RawData.Top()));

			TArray<UObject*> OuterObjects;
			StructPropertyHandle->GetOuterObjects(OuterObjects);
			ULocalizationTarget* const LocalizationTarget = Cast<ULocalizationTarget>(OuterObjects.Top());
			if (LocalizationTarget)
			{
				FText Error;
				if (!Struct.Validate(LocalizationTarget->IsMemberOfEngineTargetSet(), Error))
				{
					return Error;
				}
			}
		}

		return FText::GetEmpty();
	};

	HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SGatherTextConfigurationErrorHint)
				.ErrorText_Lambda(GetErrorText)
			]
		]
		.ValueContent()
		[
			NamePropertyHandle->CreatePropertyValueWidget()
		];
}

void FMetaDataKeyGatherSpecificationStructCustomization::CustomizeStructHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils)
{
	const auto& GetErrorText = [StructPropertyHandle]() -> FText
	{
		if (StructPropertyHandle->IsValidHandle())
		{
			TArray<const void*> RawData;
			StructPropertyHandle->AccessRawData(RawData);
			const FMetaDataKeyGatherSpecification& Struct = *(reinterpret_cast<const FMetaDataKeyGatherSpecification*>(RawData.Top()));

			TArray<UObject*> OuterObjects;
			StructPropertyHandle->GetOuterObjects(OuterObjects);
			ULocalizationTarget* const LocalizationTarget = Cast<ULocalizationTarget>(OuterObjects.Top());
			if (LocalizationTarget)
			{
				FText Error;
				if (!Struct.Validate(LocalizationTarget->IsMemberOfEngineTargetSet(), Error))
				{
					return Error;
				}
			}
		}

		return FText::GetEmpty();
	};

	HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SGatherTextConfigurationErrorHint)
				.ErrorText_Lambda(GetErrorText)
			]
		]
		.ValueContent()
		[
			StructPropertyHandle->CreatePropertyValueWidget()
		];
}

void FMetaDataKeyGatherSpecificationStructCustomization::CustomizeStructChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils )
{
	uint32 ChildCount;
	StructPropertyHandle->GetNumChildren(ChildCount);

	for (uint32 i = 0; i < ChildCount; ++i)
	{
		const TSharedPtr<IPropertyHandle> ChildPropertyHandle = StructPropertyHandle->GetChildHandle(i);
		if (!ChildPropertyHandle->IsCustomized())
		{
			ChildBuilder.AddProperty(ChildPropertyHandle.ToSharedRef());
		}
	}
}

void FGatherTextFromMetaDataConfigurationStructCustomization::CustomizeStructHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils )
{
	const TSharedPtr<IPropertyHandle> IsEnabledPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGatherTextFromMetaDataConfiguration, IsEnabled));
	if (IsEnabledPropertyHandle.IsValid() && IsEnabledPropertyHandle->IsValidHandle())
	{
		IsEnabledPropertyHandle->MarkHiddenByCustomization();
	}

	const auto& GetConfigurationError = [StructPropertyHandle]() -> FText
	{
		TArray<const void*> RawData;
		StructPropertyHandle->AccessRawData(RawData);
		const FGatherTextFromMetaDataConfiguration& Struct = *(reinterpret_cast<const FGatherTextFromMetaDataConfiguration*>(RawData.Top()));

		TArray<UObject*> OuterObjects;
		StructPropertyHandle->GetOuterObjects(OuterObjects);
		ULocalizationTarget* const LocalizationTarget = Cast<ULocalizationTarget>(OuterObjects.Top());
		if (LocalizationTarget)
		{
			FText Error;
			if (!Struct.Validate(LocalizationTarget->IsMemberOfEngineTargetSet(), Error))
			{
				return Error;
			}
		}

		return FText::GetEmpty();
	};

	const auto& GetValidityWidgetVisibility = [IsEnabledPropertyHandle]() -> EVisibility
	{
		bool Value = false;
		if (IsEnabledPropertyHandle.IsValid() && IsEnabledPropertyHandle->IsValidHandle())
		{
			IsEnabledPropertyHandle->GetValue(Value);
		}
		return Value ? EVisibility::Visible : EVisibility::Hidden;
	};
		
	HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				IsEnabledPropertyHandle.IsValid() && IsEnabledPropertyHandle->IsValidHandle() ? IsEnabledPropertyHandle->CreatePropertyValueWidget() : SNullWidget::NullWidget
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				StructPropertyHandle->CreatePropertyValueWidget()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SConfigurationValidity)
				.Visibility_Lambda(GetValidityWidgetVisibility)
				.ConfigurationError_Lambda(GetConfigurationError)
			]
		];
}

void FGatherTextFromMetaDataConfigurationStructCustomization::CustomizeStructChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils )
{
	uint32 ChildCount;
	StructPropertyHandle->GetNumChildren(ChildCount);

	for (uint32 i = 0; i < ChildCount; ++i)
	{
		const TSharedPtr<IPropertyHandle> ChildPropertyHandle = StructPropertyHandle->GetChildHandle(i);
		if (!ChildPropertyHandle->IsCustomized())
		{
			ChildBuilder.AddProperty(ChildPropertyHandle.ToSharedRef());
		}
	}
}

#undef LOCTEXT_NAMESPACE
