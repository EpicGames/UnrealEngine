// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;

class FGatherTextSearchDirectoryStructCustomization : public IStructCustomization
{
public:
	virtual void CustomizeStructHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeStructChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils) override {};
};

class FGatherTextIncludePathStructCustomization : public IStructCustomization
{
public:
	virtual void CustomizeStructHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeStructChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils) override {};
};

class FGatherTextExcludePathStructCustomization : public IStructCustomization
{
public:
	virtual void CustomizeStructHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeStructChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils) override {};
};

class FGatherTextFileExtensionStructCustomization : public IStructCustomization
{
public:
	virtual void CustomizeStructHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeStructChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils) override {};
};

class FGatherTextFromTextFilesConfigurationStructCustomization : public IStructCustomization
{
public:
	virtual void CustomizeStructHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeStructChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils) override;
};

class FGatherTextFromPackagesConfigurationStructCustomization : public IStructCustomization
{
public:
	virtual void CustomizeStructHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeStructChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils) override;
};

class FMetaDataTextKeyPatternStructCustomization : public IStructCustomization
{
public:
	virtual void CustomizeStructHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeStructChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils) override {};
};

class FMetaDataKeyNameStructCustomization : public IStructCustomization
{
public:
	virtual void CustomizeStructHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeStructChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils) override {};
};

class FMetaDataKeyGatherSpecificationStructCustomization : public IStructCustomization
{
public:
	virtual void CustomizeStructHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeStructChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils) override;
};

class FGatherTextFromMetaDataConfigurationStructCustomization : public IStructCustomization
{
public:
	virtual void CustomizeStructHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeStructChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils) override;
};
