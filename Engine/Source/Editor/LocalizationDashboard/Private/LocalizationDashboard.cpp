// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationDashboard.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/LayoutService.h"
#include "Widgets/Docking/SDockTab.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Interfaces/IMainFrameModule.h"
#include "SLocalizationTargetEditor.h"
#include "LocalizationTargetTypes.h"
#include "LocalizationSettings.h"
#include "LocalizationDashboardSettings.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "ToolMenus.h"
#include "WorkspaceMenuStructureModule.h"
#include "WorkspaceMenuStructure.h"

#define LOCTEXT_NAMESPACE "LocalizationDashboard"

const FName FLocalizationDashboard::TabName("LocalizationDashboard");

class SLocalizationDashboard : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLocalizationDashboard) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<SWindow>& OwningWindow, const TSharedRef<SDockTab>& OwningTab);

	TWeakPtr<SDockTab> ShowTargetEditor(ULocalizationTarget* const LocalizationTarget);

private:
	bool CanMakeEdits() const;

private:
	static const FName TargetsDetailsTabName;
	static const FName DocumentsTabName;

	TSharedPtr<FTabManager> TabManager;
	TMap< TWeakObjectPtr<ULocalizationTarget>, TWeakPtr<SDockTab> > TargetToTabMap;
	TSharedPtr<SSettingsEditorCheckoutNotice> SettingsEditorCheckoutNotice;
};

const FName SLocalizationDashboard::TargetsDetailsTabName("Targets");
const FName SLocalizationDashboard::DocumentsTabName("Documents");

void SLocalizationDashboard::Construct(const FArguments& InArgs, const TSharedPtr<SWindow>& OwningWindow, const TSharedRef<SDockTab>& OwningTab)
{
	TabManager = FGlobalTabmanager::Get()->NewTabManager(OwningTab);

	const auto& PersistLayout = [](const TSharedRef<FTabManager::FLayout>& LayoutToSave)
	{
		FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, LayoutToSave);
	};
	TabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateLambda(PersistLayout));

	const auto& CreateTargetsTab = [this](const FSpawnTabArgs& SpawnTabArgs) -> TSharedRef<SDockTab>
	{
		check(SpawnTabArgs.GetTabId() == TargetsDetailsTabName);

		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.Label(LOCTEXT("TargetsTabLabel", "Targets"));

		TSharedRef<SVerticalBox> VBox = SNew(SVerticalBox);
		TSharedRef<SScrollBox> ScrollBox = SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				VBox
			];

		DockTab->SetContent(ScrollBox);

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// Settings Details View
		{
			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.bAllowSearch = false; 
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			TSharedRef<IDetailsView> DetailsView = PropertyModule.CreateDetailView(DetailsViewArgs);
			DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SLocalizationDashboard::CanMakeEdits));
			DetailsView->SetObject(GetMutableDefault<ULocalizationDashboardSettings>(), true);

			VBox->AddSlot()
				.AutoHeight()
				[
					DetailsView
				];
		}

		// Game Targets Details View
		{
			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.bAllowSearch = false; 
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			TSharedRef<IDetailsView> DetailsView = PropertyModule.CreateDetailView(DetailsViewArgs);
			DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SLocalizationDashboard::CanMakeEdits));
			DetailsView->SetObject(ULocalizationSettings::GetGameTargetSet(), true);

			VBox->AddSlot()
				.AutoHeight()
				[
					DetailsView
				];
		}

		// Engine Targets Details View
		{
			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.bAllowSearch = false; 
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			TSharedRef<IDetailsView> DetailsView = PropertyModule.CreateDetailView(DetailsViewArgs);
			DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SLocalizationDashboard::CanMakeEdits));
			DetailsView->SetObject(ULocalizationSettings::GetEngineTargetSet(), true);

			VBox->AddSlot()
				.AutoHeight()
				[
					DetailsView
				];
		}

		return DockTab;
	};
	const TSharedRef<FWorkspaceItem> TargetSetsWorkspaceMenuCategory = TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("LocalizationDashboardWorkspaceMenuCategory", "Localization Dashboard"));

	FTabSpawnerEntry& TargetsTabSpawnerEntry = TabManager->RegisterTabSpawner(TargetsDetailsTabName, FOnSpawnTab::CreateLambda(CreateTargetsTab))
		.SetDisplayName(LOCTEXT("TargetsDetailTabSpawner", "Targets"));

	TargetSetsWorkspaceMenuCategory->AddItem(TargetsTabSpawnerEntry.AsShared());

	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("LocalizationDashboard_v5")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(1.0f)
				->SetHideTabWell(true)
				->AddTab(TargetsDetailsTabName, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(2.0f)
				->SetHideTabWell(false)
				->AddTab(DocumentsTabName, ETabState::ClosedTab)
			)
		);

	Layout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, Layout);

	FToolMenuContext ToolMenuContext;
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");

	TabManager->SetAllowWindowMenuBar(true);

	UToolMenus::Get()->RegisterMenu("LocalizationDashboard.MainMenu", "MainFrame.NomadMainMenu");

	FString ConfigFilePath;
	ConfigFilePath = GetDefault<ULocalizationSettings>()->GetDefaultConfigFilename();
	ConfigFilePath = FPaths::ConvertRelativePathToFull(ConfigFilePath);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f)
		[
			SAssignNew(SettingsEditorCheckoutNotice, SSettingsEditorCheckoutNotice)
			.ConfigFilePath(ConfigFilePath)
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			TabManager->RestoreFrom(Layout, OwningWindow).ToSharedRef()
		]
	];

	MainFrameModule.MakeMainMenu(TabManager, "LocalizationDashboard.MainMenu", ToolMenuContext);

	// Open the first game target if present
	if (ULocalizationTargetSet* GameTargetSet = ULocalizationSettings::GetGameTargetSet())
	{
		if (GameTargetSet->TargetObjects.Num() > 0)
		{
			ShowTargetEditor(GameTargetSet->TargetObjects[0]);
		}
	}
}

TWeakPtr<SDockTab> SLocalizationDashboard::ShowTargetEditor(ULocalizationTarget* const LocalizationTarget)
{
	// Create tab if not existent.
	TWeakPtr<SDockTab>& TargetEditorDockTab = TargetToTabMap.FindOrAdd(MakeWeakObjectPtr(LocalizationTarget));

	if (!TargetEditorDockTab.IsValid())
	{
		ULocalizationTargetSet* const TargetSet = LocalizationTarget->GetTypedOuter<ULocalizationTargetSet>();

		const TSharedRef<SLocalizationTargetEditor> OurTargetEditor = SNew(SLocalizationTargetEditor, TargetSet, LocalizationTarget, FIsPropertyEditingEnabled::CreateSP(this, &SLocalizationDashboard::CanMakeEdits));
		const TSharedRef<SDockTab> NewTargetEditorTab = SNew(SDockTab)
			.TabRole(ETabRole::DocumentTab)
			.Label_Lambda( [LocalizationTarget]
			{
				return LocalizationTarget ? FText::FromString(LocalizationTarget->Settings.Name) : FText::GetEmpty();
			})
			[
				OurTargetEditor
			];

		TabManager->InsertNewDocumentTab(DocumentsTabName, FTabManager::ESearchPreference::RequireClosedTab, NewTargetEditorTab);
		TargetEditorDockTab = NewTargetEditorTab;
	}
	else
	{
		const TSharedPtr<SDockTab> OldTargetEditorTab = TargetEditorDockTab.Pin();
		TabManager->DrawAttention(OldTargetEditorTab.ToSharedRef());
	}

	return TargetEditorDockTab;
}

bool SLocalizationDashboard::CanMakeEdits() const
{
	return SettingsEditorCheckoutNotice.IsValid() && SettingsEditorCheckoutNotice->IsUnlocked();
}

FLocalizationDashboard* FLocalizationDashboard::Instance = nullptr;

FLocalizationDashboard* FLocalizationDashboard::Get()
{
	return Instance;
}

void FLocalizationDashboard::Initialize()
{
	if (!Instance)
	{
		Instance = new FLocalizationDashboard();
	}
}

void FLocalizationDashboard::Terminate()
{
	if (Instance)
	{
		delete Instance;
	}
}

void FLocalizationDashboard::Show()
{
	FGlobalTabmanager::Get()->TryInvokeTab(TabName);
}

TWeakPtr<SDockTab> FLocalizationDashboard::ShowTargetEditorTab(ULocalizationTarget* const LocalizationTarget)
{
	if (LocalizationDashboardWidget.IsValid())
	{
		return LocalizationDashboardWidget->ShowTargetEditor(LocalizationTarget);
	}
	return nullptr;
}

FLocalizationDashboard::FLocalizationDashboard()
{
	RegisterTabSpawner();
}

FLocalizationDashboard::~FLocalizationDashboard()
{
	UnregisterTabSpawner();
}

void FLocalizationDashboard::RegisterTabSpawner()
{
	auto SpawnMainTab = [this](const FSpawnTabArgs& Args) -> TSharedRef<SDockTab>
	{
		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.Label(LOCTEXT("MainTabTitle", "Localization Dashboard"))
			.TabRole(ETabRole::MajorTab);

		DockTab->SetContent( SAssignNew(LocalizationDashboardWidget, SLocalizationDashboard, Args.GetOwnerWindow(), DockTab) );

		return DockTab;
	};

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TabName, FOnSpawnTab::CreateLambda(MoveTemp(SpawnMainTab)))
		.SetDisplayName(LOCTEXT("MainTabTitle", "Localization Dashboard"))
		.SetTooltipText(LOCTEXT("LocalizationDashboardToolTip", "Open the Localization Dashboard for this Project."))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationDashboard.MenuIcon"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory());
}

void FLocalizationDashboard::UnregisterTabSpawner()
{
	FGlobalTabmanager::Get()->UnregisterTabSpawner(TabName);
}

#undef LOCTEXT_NAMESPACE
