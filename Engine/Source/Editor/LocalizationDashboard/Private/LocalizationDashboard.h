// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"

class SLocalizationDashboard;
class ULocalizationTarget;

class FLocalizationDashboard
{
public:
	static FLocalizationDashboard* Get();
	static void Initialize();
	static void Terminate();
	void Show();
	TWeakPtr<SDockTab> ShowTargetEditorTab(ULocalizationTarget* const LocalizationTarget);

private:
	FLocalizationDashboard();
	~FLocalizationDashboard();
	void RegisterTabSpawner();
	void UnregisterTabSpawner();
	
private:
	static const FName TabName;
	static FLocalizationDashboard* Instance;
	TSharedPtr<SLocalizationDashboard> LocalizationDashboardWidget;
};
