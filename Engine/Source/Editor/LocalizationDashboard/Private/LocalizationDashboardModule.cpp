// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Features/IModularFeatures.h"
#include "GatherTextDetailCustomizations.h"
#include "HAL/PlatformCrt.h"
#include "ILocalizationDashboardModule.h"
#include "ILocalizationServiceModule.h"
#include "ILocalizationServiceProvider.h"
#include "LocalizationDashboard.h"
#include "LocalizationDashboardSettingsDetailCustomization.h"
#include "LocalizationTargetDetailCustomization.h"
#include "LocalizationTargetSetDetailCustomization.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class IDetailCustomization;
class IStructCustomization;

#define LOCTEXT_NAMESPACE "LocalizationDashboard"

class FLocalizationDashboardModule
	: public ILocalizationDashboardModule
{
public:
	// Begin IModuleInterface interface
	virtual void StartupModule() override
	{
		// Ensure Localization Service module is loaded (registers default localization service on startup)
		FModuleManager::LoadModuleChecked<ILocalizationServiceModule>("LocalizationService");
		ServiceProviders = IModularFeatures::Get().GetModularFeatureImplementations<ILocalizationServiceProvider>("LocalizationService");

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout("LocalizationDashboardSettings", FOnGetDetailCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IDetailCustomization>
			{
				return MakeShareable( new FLocalizationDashboardSettingsDetailCustomization());
			})
		);
		PropertyModule.RegisterCustomClassLayout("LocalizationTargetSet", FOnGetDetailCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IDetailCustomization>
			{
				return MakeShareable( new FLocalizationTargetSetDetailCustomization());
			})
		);
		PropertyModule.RegisterCustomClassLayout("LocalizationTarget", FOnGetDetailCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IDetailCustomization>
			{
				return MakeShareable( new FLocalizationTargetDetailCustomization());
			})
		);
		PropertyModule.RegisterCustomPropertyTypeLayout("GatherTextSearchDirectory", FOnGetPropertyTypeCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IStructCustomization>
			{
				return MakeShareable( new FGatherTextSearchDirectoryStructCustomization());
			})
		);
		PropertyModule.RegisterCustomPropertyTypeLayout("GatherTextIncludePath", FOnGetPropertyTypeCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IStructCustomization>
			{
				return MakeShareable( new FGatherTextIncludePathStructCustomization());
			})
		);
		PropertyModule.RegisterCustomPropertyTypeLayout("GatherTextExcludePath", FOnGetPropertyTypeCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IStructCustomization>
			{
				return MakeShareable( new FGatherTextExcludePathStructCustomization());
			})
		);
		PropertyModule.RegisterCustomPropertyTypeLayout("GatherTextFileExtension", FOnGetPropertyTypeCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IStructCustomization>
			{
				return MakeShareable( new FGatherTextFileExtensionStructCustomization());
			})
		);
		PropertyModule.RegisterCustomPropertyTypeLayout("GatherTextFromTextFilesConfiguration", FOnGetPropertyTypeCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IStructCustomization>
			{
				return MakeShareable( new FGatherTextFromTextFilesConfigurationStructCustomization());
			})
		);
		PropertyModule.RegisterCustomPropertyTypeLayout("GatherTextFromPackagesConfiguration", FOnGetPropertyTypeCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IStructCustomization>
			{
				return MakeShareable( new FGatherTextFromPackagesConfigurationStructCustomization());
			})
		);
		PropertyModule.RegisterCustomPropertyTypeLayout("MetaDataTextKeyPattern", FOnGetPropertyTypeCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IStructCustomization>
			{
				return MakeShareable( new FMetaDataTextKeyPatternStructCustomization());
			})
		);
		PropertyModule.RegisterCustomPropertyTypeLayout("MetaDataKeyName", FOnGetPropertyTypeCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IStructCustomization>
			{
				return MakeShareable( new FMetaDataKeyNameStructCustomization());
			})
		);
		PropertyModule.RegisterCustomPropertyTypeLayout("MetaDataKeyGatherSpecification", FOnGetPropertyTypeCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IStructCustomization>
			{
				return MakeShareable( new FMetaDataKeyGatherSpecificationStructCustomization());
			})
		);
		PropertyModule.RegisterCustomPropertyTypeLayout("GatherTextFromMetaDataConfiguration", FOnGetPropertyTypeCustomizationInstance::CreateLambda(
			[]() -> TSharedRef<IStructCustomization>
			{
				return MakeShareable( new FGatherTextFromMetaDataConfigurationStructCustomization());
			})
		);
			
		FLocalizationDashboard::Initialize();
	}

	virtual void ShutdownModule() override
	{
		FLocalizationDashboard::Terminate();

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GatherTextFromMetaDataConfiguration");
		PropertyModule.UnregisterCustomPropertyTypeLayout("MetaDataKeyGatherSpecification");
		PropertyModule.UnregisterCustomPropertyTypeLayout("MetaDataKeyName");
		PropertyModule.UnregisterCustomPropertyTypeLayout("MetaDataTextKeyPattern");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GatherTextFromPackagesConfiguration");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GatherTextFromTextFilesConfiguration");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GatherTextFileExtension");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GatherTextExcludePath");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GatherTextIncludePath");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GatherTextSearchDirectory");
		PropertyModule.UnregisterCustomClassLayout("LocalizationTarget");
		PropertyModule.UnregisterCustomClassLayout("LocalizationTargetSet");
		PropertyModule.UnregisterCustomClassLayout("LocalizationDashboardSettings");
	}
	// End IModuleInterface interface

	virtual void Show() override
	{
		FLocalizationDashboard* const LocalizationDashboard = FLocalizationDashboard::Get();
		if (LocalizationDashboard)
		{
			LocalizationDashboard->Show();
		}
	}

	virtual const TArray<ILocalizationServiceProvider*>& GetLocalizationServiceProviders() const override
	{
		return ServiceProviders;
	}

	virtual ILocalizationServiceProvider* GetCurrentLocalizationServiceProvider() const override
	{
		if(CurrentServiceProviderName == NAME_None)
		{
			return nullptr;
		}

		auto ServiceProviderNameComparator = [this](ILocalizationServiceProvider* const LSP) -> bool
		{
			return LSP ? LSP->GetName() == CurrentServiceProviderName : false;
		};
		ILocalizationServiceProvider* const * LSP = ServiceProviders.FindByPredicate(ServiceProviderNameComparator);
		return LSP ? *LSP : nullptr;
	}

private:
	TArray<ILocalizationServiceProvider*> ServiceProviders;
	FName CurrentServiceProviderName;
};

IMPLEMENT_MODULE(FLocalizationDashboardModule, LocalizationDashboard);

#undef LOCTEXT_NAMESPACE
