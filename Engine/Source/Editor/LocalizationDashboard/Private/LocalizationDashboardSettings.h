// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

#include "LocalizationDashboardSettings.generated.h"

/** Currently just a dummy class used for detail customizations */
UCLASS()
class ULocalizationDashboardSettings : public UObject
{
	GENERATED_BODY()
};
