// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationDashboardSettingsDetailCustomization.h"

#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ILocalizationDashboardModule.h"
#include "ILocalizationServiceModule.h"
#include "ILocalizationServiceProvider.h"
#include "Internationalization/Internationalization.h"
#include "LocalizationSettings.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Styling/SlateTypes.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "LocalizationDashboard"

FLocalizationDashboardSettingsDetailCustomization::FLocalizationDashboardSettingsDetailCustomization()
	: DetailLayoutBuilder(nullptr)
	, ServiceProviderCategoryBuilder(nullptr)
{
	TArray<ILocalizationServiceProvider*> ActualProviders = ILocalizationDashboardModule::Get().GetLocalizationServiceProviders();
	for (ILocalizationServiceProvider* ActualProvider : ActualProviders)
	{
		TSharedPtr<FLocalizationServiceProviderWrapper> Provider = MakeShareable(new FLocalizationServiceProviderWrapper(ActualProvider));
		Providers.Add(Provider);
	}
}

void FLocalizationDashboardSettingsDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailLayoutBuilder = &DetailBuilder;

	// Localization Service Provider
	{
		ServiceProviderCategoryBuilder = &DetailLayoutBuilder->EditCategory("ServiceProvider", LOCTEXT("LocalizationServiceProvider", "Localization Service Provider"), ECategoryPriority::Important);
		FDetailWidgetRow& DetailWidgetRow = ServiceProviderCategoryBuilder->AddCustomRow(LOCTEXT("SelectedLocalizationServiceProvider", "Selected Localization Service Provider"));

		int32 CurrentlySelectedProviderIndex = 0;

		for (int ProviderIndex = 0; ProviderIndex < Providers.Num(); ++ProviderIndex)
		{
			FName CurrentlySelectedProviderName = ILocalizationServiceModule::Get().GetProvider().GetName();
			if (Providers[ProviderIndex].IsValid() && Providers[ProviderIndex]->Provider && Providers[ProviderIndex]->Provider->GetName() == CurrentlySelectedProviderName)
			{
				CurrentlySelectedProviderIndex = ProviderIndex;
				break;
			}
		}

		DetailWidgetRow.NameContent()
			[
				SNew(STextBlock)
				.Font(DetailLayoutBuilder->GetDetailFont())
				.Text(LOCTEXT("LocalizationServiceProvider", "Localization Service Provider"))
			];
		DetailWidgetRow.ValueContent()
			.MinDesiredWidth(TOptional<float>())
			.MaxDesiredWidth(TOptional<float>())
			[
				SNew(SComboBox< TSharedPtr<FLocalizationServiceProviderWrapper>>)
				.OptionsSource(&(Providers))
				.OnSelectionChanged(this, &FLocalizationDashboardSettingsDetailCustomization::ServiceProviderComboBox_OnSelectionChanged)
				.OnGenerateWidget(this, &FLocalizationDashboardSettingsDetailCustomization::ServiceProviderComboBox_OnGenerateWidget)
				.InitiallySelectedItem(Providers[CurrentlySelectedProviderIndex])
				.Content()
				[
					SNew(STextBlock)
					.Font(DetailLayoutBuilder->GetDetailFont())
					.Text_Lambda([]()
					{
						return ILocalizationServiceModule::Get().GetProvider().GetDisplayName();
					})
				]
			];

#if LOCALIZATION_SERVICES_WITH_SLATE
		const ILocalizationServiceProvider& LSP = ILocalizationServiceModule::Get().GetProvider();
		if (ServiceProviderCategoryBuilder != nullptr)
		{
			LSP.CustomizeSettingsDetails(*ServiceProviderCategoryBuilder);
		}
#endif
	}

	// Source Control
	{
		IDetailCategoryBuilder& SourceControlCategoryBuilder = DetailLayoutBuilder->EditCategory("SourceControl", LOCTEXT("SourceControl", "Revision Control"), ECategoryPriority::Important);

		// Enable Source Control
		{
			SourceControlCategoryBuilder.AddCustomRow(LOCTEXT("EnableSourceControl", "Enable Revision Control"))
				.NameContent()
				[
					SNew(STextBlock)
					.Font(DetailLayoutBuilder->GetDetailFont())
					.Text(LOCTEXT("EnableSourceControl", "Enable Revision Control"))
					.ToolTipText(LOCTEXT("EnableSourceControlToolTip", "Should we use revision control when running the localization commandlets. This will optionally pass \"-EnableSCC\" to the commandlet."))
				]
				.ValueContent()
				.MinDesiredWidth(TOptional<float>())
				.MaxDesiredWidth(TOptional<float>())
				[
					SNew(SCheckBox)
					.ToolTipText(LOCTEXT("EnableSourceControlToolTip", "Should we use revision control when running the localization commandlets. This will optionally pass \"-EnableSCC\" to the commandlet."))
					.IsEnabled_Lambda([]() -> bool
					{
						return FLocalizationSourceControlSettings::IsSourceControlAvailable();
					})
					.IsChecked_Lambda([]() -> ECheckBoxState
					{
						return FLocalizationSourceControlSettings::IsSourceControlEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([](ECheckBoxState InCheckState)
					{
						FLocalizationSourceControlSettings::SetSourceControlEnabled(InCheckState == ECheckBoxState::Checked);
					})
				];
		}

		// Enable Auto Submit
		{
			SourceControlCategoryBuilder.AddCustomRow(LOCTEXT("EnableSourceControlAutoSubmit", "Enable Auto Submit"))
				.NameContent()
				[
					SNew(STextBlock)
					.Font(DetailLayoutBuilder->GetDetailFont())
					.Text(LOCTEXT("EnableSourceControlAutoSubmit", "Enable Auto Submit"))
					.ToolTipText(LOCTEXT("EnableSourceControlAutoSubmitToolTip", "Should we automatically submit changed files after running the commandlet. This will optionally pass \"-DisableSCCSubmit\" to the commandlet."))
				]
				.ValueContent()
				.MinDesiredWidth(TOptional<float>())
				.MaxDesiredWidth(TOptional<float>())
				[
					SNew(SCheckBox)
					.ToolTipText(LOCTEXT("EnableSourceControlAutoSubmitToolTip", "Should we automatically submit changed files after running the commandlet. This will optionally pass \"-DisableSCCSubmit\" to the commandlet."))
					.IsEnabled_Lambda([]() -> bool
					{
						return FLocalizationSourceControlSettings::IsSourceControlAvailable();
					})
					.IsChecked_Lambda([]() -> ECheckBoxState
					{
						return FLocalizationSourceControlSettings::IsSourceControlAutoSubmitEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([](ECheckBoxState InCheckState)
					{
						FLocalizationSourceControlSettings::SetSourceControlAutoSubmitEnabled(InCheckState == ECheckBoxState::Checked);
					})
				];
		}
	}
}

FText FLocalizationDashboardSettingsDetailCustomization::GetCurrentServiceProviderDisplayName() const
{
	const ILocalizationServiceProvider& LSP = ILocalizationServiceModule::Get().GetProvider();
	return LSP.GetDisplayName();
}

TSharedRef<SWidget> FLocalizationDashboardSettingsDetailCustomization::ServiceProviderComboBox_OnGenerateWidget(TSharedPtr<FLocalizationServiceProviderWrapper> LSPWrapper) const
{
	ILocalizationServiceProvider* LSP = LSPWrapper.IsValid() ? LSPWrapper->Provider : nullptr;

	return	SNew(STextBlock)
		.Text(LSP ? LSP->GetDisplayName() : LOCTEXT("NoServiceProviderName", "None"));
}

void FLocalizationDashboardSettingsDetailCustomization::ServiceProviderComboBox_OnSelectionChanged(TSharedPtr<FLocalizationServiceProviderWrapper> LSPWrapper, ESelectInfo::Type SelectInfo)
{
	ILocalizationServiceProvider* LSP = LSPWrapper.IsValid() ? LSPWrapper->Provider : nullptr;

	FName ServiceProviderName = LSP ? LSP->GetName() : FName(TEXT("None"));
	ILocalizationServiceModule::Get().SetProvider(ServiceProviderName);

#if LOCALIZATION_SERVICES_WITH_SLATE
	if (LSP && ServiceProviderCategoryBuilder)
	{
		LSP->CustomizeSettingsDetails(*ServiceProviderCategoryBuilder);
	}
#endif
	DetailLayoutBuilder->ForceRefreshDetails();
}

#undef LOCTEXT_NAMESPACE
