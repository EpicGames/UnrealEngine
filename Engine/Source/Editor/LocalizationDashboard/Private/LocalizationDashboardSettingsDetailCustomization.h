// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"

class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class ILocalizationServiceProvider;
class SWidget;

struct FLocalizationServiceProviderWrapper
{
	FLocalizationServiceProviderWrapper() : Provider(nullptr) {}
	FLocalizationServiceProviderWrapper(ILocalizationServiceProvider* const InProvider) : Provider(InProvider) {}

	ILocalizationServiceProvider* Provider;
};

class FLocalizationDashboardSettingsDetailCustomization : public IDetailCustomization
{
public:
	FLocalizationDashboardSettingsDetailCustomization();
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	FText GetCurrentServiceProviderDisplayName() const;
	TSharedRef<SWidget> ServiceProviderComboBox_OnGenerateWidget(TSharedPtr<FLocalizationServiceProviderWrapper> LSPWrapper) const;
	void ServiceProviderComboBox_OnSelectionChanged(TSharedPtr<FLocalizationServiceProviderWrapper> LSPWrapper, ESelectInfo::Type SelectInfo);

private:
	IDetailLayoutBuilder* DetailLayoutBuilder;

	IDetailCategoryBuilder* ServiceProviderCategoryBuilder;
	TArray< TSharedPtr<FLocalizationServiceProviderWrapper> > Providers;
};
