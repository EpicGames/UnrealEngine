// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationTargetDetailCustomization.h"

#include "CoreGlobals.h"
#include "DesktopPlatformModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "FileHelpers.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Views/ITypedTableView.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "GenericPlatform/GenericWindow.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformMisc.h"
#include "IDesktopPlatform.h"
#include "IDetailsView.h"
#include "ILocalizationServiceModule.h"
#include "ILocalizationServiceProvider.h"
#include "ISourceControlModule.h"
#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Internationalization/Culture.h"
#include "Layout/Margin.h"
#include "LocalizationCommandletTasks.h"
#include "LocalizationConfigurationScript.h"
#include "LocalizationSettings.h"
#include "LocalizationTargetTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/Guid.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "ObjectEditorUtils.h"
#include "PropertyHandle.h"
#include "SCulturePicker.h"
#include "SLocalizationTargetEditorCultureRow.h"
#include "SLocalizationTargetStatusButton.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "SourceControlOperations.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "LocalizationTargetEditor"

namespace
{
	struct FLocalizationTargetLoadingPolicyConfig
	{
		FLocalizationTargetLoadingPolicyConfig(ELocalizationTargetLoadingPolicy InLoadingPolicy, FString InSectionName, FString InKeyName, FString InConfigName, FString InConfigPath)
			: LoadingPolicy(InLoadingPolicy)
			, SectionName(MoveTemp(InSectionName))
			, KeyName(MoveTemp(InKeyName))
			, BaseConfigName(MoveTemp(InConfigName))
			, ConfigPath(MoveTemp(InConfigPath))
		{
			DefaultConfigName = FString::Printf(TEXT("Default%s"), *BaseConfigName);
			DefaultConfigFilePath = FString::Printf(TEXT("%s%s.ini"), *FPaths::SourceConfigDir(), *DefaultConfigName);
		}

		ELocalizationTargetLoadingPolicy LoadingPolicy;
		FString SectionName;
		FString KeyName;
		FString BaseConfigName;
		FString DefaultConfigName;
		FString DefaultConfigFilePath;
		FString ConfigPath;
	};

	static const TArray<FLocalizationTargetLoadingPolicyConfig> LoadingPolicyConfigs = []()
	{
		TArray<FLocalizationTargetLoadingPolicyConfig> Array;
		Array.Emplace(ELocalizationTargetLoadingPolicy::Always,			TEXT("Internationalization"),	TEXT("LocalizationPaths"),				TEXT("Engine"),		GEngineIni);
		Array.Emplace(ELocalizationTargetLoadingPolicy::Editor,			TEXT("Internationalization"),	TEXT("LocalizationPaths"),				TEXT("Editor"),		GEditorIni);
		Array.Emplace(ELocalizationTargetLoadingPolicy::Game,			TEXT("Internationalization"),	TEXT("LocalizationPaths"),				TEXT("Game"),		GGameIni);
		Array.Emplace(ELocalizationTargetLoadingPolicy::PropertyNames,	TEXT("Internationalization"),	TEXT("PropertyNameLocalizationPaths"),	TEXT("Editor"),		GEditorIni);
		Array.Emplace(ELocalizationTargetLoadingPolicy::ToolTips,		TEXT("Internationalization"),	TEXT("ToolTipLocalizationPaths"),		TEXT("Editor"),		GEditorIni);
		return Array;
	}();
}

FLocalizationTargetDetailCustomization::FLocalizationTargetDetailCustomization()
	: DetailLayoutBuilder(nullptr)
	, NewEntryIndexToBeInitialized(INDEX_NONE)
{
}

class FLocalizationTargetEditorCommands : public TCommands<FLocalizationTargetEditorCommands>
{
public:
	FLocalizationTargetEditorCommands() 
		: TCommands<FLocalizationTargetEditorCommands>("LocalizationTargetEditor", NSLOCTEXT("Contexts", "LocalizationTargetEditor", "Localization Target Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	TSharedPtr<FUICommandInfo> GatherText;
	TSharedPtr<FUICommandInfo> ImportTextAllCultures;
	TSharedPtr<FUICommandInfo> ExportTextAllCultures;
	TSharedPtr<FUICommandInfo> ImportDialogueScriptAllCultures;
	TSharedPtr<FUICommandInfo> ExportDialogueScriptAllCultures;
	TSharedPtr<FUICommandInfo> ImportDialogueAllCultures;
	TSharedPtr<FUICommandInfo> CountWords;
	TSharedPtr<FUICommandInfo> CompileTextAllCultures;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};

void FLocalizationTargetEditorCommands::RegisterCommands() 
{
	UI_COMMAND(GatherText, "Gather Text", "Gather text for all languages of this target.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ImportTextAllCultures, "Import Text", "Import translations for all languages of this target.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportTextAllCultures, "Export Text", "Export translations for all languages of this target.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ImportDialogueScriptAllCultures, "Import Script", "Import dialogue scripts for all languages of this target.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportDialogueScriptAllCultures, "Export Script", "Export dialogue scripts for all languages of this target.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ImportDialogueAllCultures, "Import Dialogue", "Import dialogue WAV files for all languages of this target.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CountWords, "Count Words", "Count translations for all languages of this target.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CompileTextAllCultures, "Compile Text", "Compile translations for all languages of this target.", EUserInterfaceActionType::Button, FInputChord());
}

void FLocalizationTargetDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailLayoutBuilder = &DetailBuilder;
	{
		TArray< TWeakObjectPtr<UObject> > ObjectsBeingCustomized;
		DetailLayoutBuilder->GetObjectsBeingCustomized(ObjectsBeingCustomized);
		LocalizationTarget = CastChecked<ULocalizationTarget>(ObjectsBeingCustomized.Top().Get());
		TargetSet = CastChecked<ULocalizationTargetSet>(LocalizationTarget->GetOuter());
	}

	const ILocalizationServiceProvider& LSP = ILocalizationServiceModule::Get().GetProvider();

	TargetSettingsPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULocalizationTarget, Settings));

	typedef TFunction<void(const TSharedRef<IPropertyHandle>&, IDetailCategoryBuilder&)> FPropertyCustomizationFunction;
	TMap<FName, FPropertyCustomizationFunction> PropertyCustomizationMap;

	PropertyCustomizationMap.Add(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, Name), [&](const TSharedRef<IPropertyHandle>& MemberPropertyHandle, IDetailCategoryBuilder& DetailCategoryBuilder)
	{
		FDetailWidgetRow& DetailWidgetRow = DetailCategoryBuilder.AddCustomRow( MemberPropertyHandle->GetPropertyDisplayName() );
		DetailWidgetRow.NameContent()
			[
				MemberPropertyHandle->CreatePropertyNameWidget()
			];
		DetailWidgetRow.ValueContent()
			[
				SAssignNew(TargetNameEditableTextBox, SEditableTextBox)
				.Font(DetailLayoutBuilder->GetDetailFont())
				.Text(this, &FLocalizationTargetDetailCustomization::GetTargetName)
				.RevertTextOnEscape(true)
				.OnTextChanged(this, &FLocalizationTargetDetailCustomization::OnTargetNameChanged)
				.OnTextCommitted(this, &FLocalizationTargetDetailCustomization::OnTargetNameCommitted)
			];
	});

	PropertyCustomizationMap.Add(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, ConflictStatus), [&](const TSharedRef<IPropertyHandle>& MemberPropertyHandle, IDetailCategoryBuilder& DetailCategoryBuilder)
	{
		FDetailWidgetRow& StatusRow = DetailCategoryBuilder.AddCustomRow( MemberPropertyHandle->GetPropertyDisplayName() );
		StatusRow.NameContent()
			[
				MemberPropertyHandle->CreatePropertyNameWidget()
			];
		StatusRow.ValueContent()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SLocalizationTargetStatusButton, *LocalizationTarget)
			];
	});

	PropertyCustomizationMap.Add(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, TargetDependencies), [&](const TSharedRef<IPropertyHandle>& MemberPropertyHandle, IDetailCategoryBuilder& DetailCategoryBuilder)
	{
		const auto& MenuContentLambda = [this]() -> TSharedRef<SWidget>
		{
			RebuildTargetsList();

			if (TargetDependenciesOptionsList.Num() > 0)
			{
				return SNew(SBox)
					.MaxDesiredHeight(400.0f)
					.MaxDesiredWidth(300.0f)
					[
						SNew(SListView<ULocalizationTarget*>)
						.SelectionMode(ESelectionMode::None)
						.ListItemsSource(&TargetDependenciesOptionsList)
						.OnGenerateRow(this, &FLocalizationTargetDetailCustomization::OnGenerateTargetRow)
					];
			}
			else
			{
				return SNullWidget::NullWidget;
			}
		};

		FDetailWidgetRow& TargetDependenciesRow = DetailCategoryBuilder.AddCustomRow( MemberPropertyHandle->GetPropertyDisplayName() );
		TargetDependenciesRow.NameContent()
			[
				MemberPropertyHandle->CreatePropertyNameWidget()
			];
		TargetDependenciesRow.ValueContent()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SComboButton)
				.ContentPadding(FMargin(4.0, 2.0))
				.ButtonContent()
				[
					SAssignNew(TargetDependenciesHorizontalBox, SHorizontalBox)
				]
				.HasDownArrow(true)
				.OnGetMenuContent_Lambda(MenuContentLambda)
			];

		RebuildTargetDependenciesBox();
	});

	PropertyCustomizationMap.Add(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, NativeCultureIndex), [&](const TSharedRef<IPropertyHandle>& MemberPropertyHandle, IDetailCategoryBuilder& DetailCategoryBuilder)
	{
		NativeCultureIndexPropertyHandle = MemberPropertyHandle;
	});

	PropertyCustomizationMap.Add(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, SupportedCulturesStatistics), [&](const TSharedRef<IPropertyHandle>& MemberPropertyHandle, IDetailCategoryBuilder& DetailCategoryBuilder)
	{

		SupportedCulturesStatisticsPropertyHandle = MemberPropertyHandle;

		SupportedCulturesStatisticsPropertyHandle_OnNumElementsChanged = FSimpleDelegate::CreateSP(this, &FLocalizationTargetDetailCustomization::RebuildListedCulturesList);
		SupportedCulturesStatisticsPropertyHandle->AsArray()->SetOnNumElementsChanged(SupportedCulturesStatisticsPropertyHandle_OnNumElementsChanged);

		FLocalizationTargetEditorCommands::Register();
		auto Commands = FLocalizationTargetEditorCommands::Get();
		const TSharedRef< FUICommandList > CommandList = MakeShareable(new FUICommandList);
		// Let the localization service extend this toolbar
		TSharedRef<FExtender> LocalizationServiceExtender = MakeShareable(new FExtender);
#if LOCALIZATION_SERVICES_WITH_SLATE
		if (LocalizationTarget.IsValid() && ILocalizationServiceModule::Get().IsEnabled())
		{
			LSP.CustomizeTargetToolbar(LocalizationServiceExtender, LocalizationTarget);
		}
#endif
		FToolBarBuilder ToolBarBuilder(CommandList, FMultiBoxCustomization::AllowCustomization("LocalizationTargetEditor"), LocalizationServiceExtender);

		TAttribute<FText> GatherToolTipTextAttribute = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([this]() -> FText
		{
			return CanGatherText() ? FLocalizationTargetEditorCommands::Get().GatherText->GetDescription() : LOCTEXT("GatherDisabledToolTip", "Must have a native language specified in order to gather.");
		}));
		CommandList->MapAction(FLocalizationTargetEditorCommands::Get().GatherText, FExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::GatherText), FCanExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::CanGatherText));
		ToolBarBuilder.AddToolBarButton(FLocalizationTargetEditorCommands::Get().GatherText, NAME_None, TAttribute<FText>(), GatherToolTipTextAttribute, FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationTargetEditor.GatherText"));

		CommandList->MapAction(FLocalizationTargetEditorCommands::Get().ImportTextAllCultures, FExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::ImportTextAllCultures), FCanExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::CanImportTextAllCultures));
		ToolBarBuilder.AddToolBarButton(FLocalizationTargetEditorCommands::Get().ImportTextAllCultures, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationTargetEditor.ImportTextAllCultures"));

		CommandList->MapAction(FLocalizationTargetEditorCommands::Get().ExportTextAllCultures, FExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::ExportTextAllCultures), FCanExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::CanExportTextAllCultures));
		ToolBarBuilder.AddToolBarButton(FLocalizationTargetEditorCommands::Get().ExportTextAllCultures, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationTargetEditor.ExportTextAllCultures"));

		CommandList->MapAction(FLocalizationTargetEditorCommands::Get().ImportDialogueScriptAllCultures, FExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::ImportDialogueScriptAllCultures), FCanExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::CanImportDialogueScriptAllCultures));
		ToolBarBuilder.AddToolBarButton(FLocalizationTargetEditorCommands::Get().ImportDialogueScriptAllCultures, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationTargetEditor.ImportDialogueScriptAllCultures"));

		CommandList->MapAction(FLocalizationTargetEditorCommands::Get().ExportDialogueScriptAllCultures, FExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::ExportDialogueScriptAllCultures), FCanExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::CanExportDialogueScriptAllCultures));
		ToolBarBuilder.AddToolBarButton(FLocalizationTargetEditorCommands::Get().ExportDialogueScriptAllCultures, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationTargetEditor.ExportDialogueScriptAllCultures"));

		CommandList->MapAction(FLocalizationTargetEditorCommands::Get().ImportDialogueAllCultures, FExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::ImportDialogueAllCultures), FCanExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::CanImportDialogueAllCultures));
		ToolBarBuilder.AddToolBarButton(FLocalizationTargetEditorCommands::Get().ImportDialogueAllCultures, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationTargetEditor.ImportDialogueAllCultures"));

		CommandList->MapAction(FLocalizationTargetEditorCommands::Get().CountWords, FExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::CountWords), FCanExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::CanCountWords));
		ToolBarBuilder.AddToolBarButton(FLocalizationTargetEditorCommands::Get().CountWords, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationTargetEditor.CountWords"));

		CommandList->MapAction(FLocalizationTargetEditorCommands::Get().CompileTextAllCultures, FExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::CompileTextAllCultures), FCanExecuteAction::CreateSP(this, &FLocalizationTargetDetailCustomization::CanCompileTextAllCultures));
		ToolBarBuilder.AddToolBarButton(FLocalizationTargetEditorCommands::Get().CompileTextAllCultures, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationTargetEditor.CompileTextAllCultures"));

		if (ILocalizationServiceModule::Get().IsEnabled())
		{
			ToolBarBuilder.BeginSection("LocalizationService");
			ToolBarBuilder.EndSection();
		}

		BuildListedCulturesList();

		DetailCategoryBuilder.AddCustomRow( SupportedCulturesStatisticsPropertyHandle->GetPropertyDisplayName() )
			.WholeRowContent()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					ToolBarBuilder.MakeWidget()
				]
				+SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(SupportedCultureListView, SListView< TSharedPtr<IPropertyHandle> >)
						.OnGenerateRow(this, &FLocalizationTargetDetailCustomization::OnGenerateCultureRow)
						.ListItemsSource(&ListedCultureStatisticProperties)
						.SelectionMode(ESelectionMode::None)
						.HeaderRow
						(
						SNew(SHeaderRow)
						+SHeaderRow::Column("IsNative")
						.DefaultLabel( NSLOCTEXT("LocalizationCulture", "IsNativeColumnLabel", "Native"))
						.HAlignHeader(HAlign_Center)
						.HAlignCell(HAlign_Center)
						.VAlignCell(VAlign_Center)
						.FillWidth(0.1f)
						+SHeaderRow::Column("Culture")
						.DefaultLabel( NSLOCTEXT("LocalizationCulture", "CultureColumnLabel", "Language"))
						.HAlignHeader(HAlign_Fill)
						.HAlignCell(HAlign_Fill)
						.VAlignCell(VAlign_Center)
						.FillWidth(0.2f)
						+SHeaderRow::Column("WordCount")
						.DefaultLabel( NSLOCTEXT("LocalizationCulture", "WordCountColumnLabel", "Word Count"))
						.HAlignHeader(HAlign_Center)
						.HAlignCell(HAlign_Fill)
						.VAlignCell(VAlign_Center)
						.FillWidth(0.4f)
						+SHeaderRow::Column("Actions")
						.DefaultLabel( NSLOCTEXT("LocalizationCulture", "ActionsColumnLabel", "Actions"))
						.HAlignHeader(HAlign_Center)
						.HAlignCell(HAlign_Center)
						.VAlignCell(VAlign_Center)
						.FillWidth(0.3f)
						)
					]
				+SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Center)
					[
						SAssignNew(NoSupportedCulturesErrorText, SErrorText)
					]
				+SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Center)
					[
						SAssignNew(AddNewSupportedCultureComboButton, SComboButton)
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text(NSLOCTEXT("LocalizationCulture", "AddNewCultureButtonLabel", "Add New Language"))
						]
						.MenuContent()
							[
								SNew(SBox)
								.MaxDesiredHeight(400.0f)
								.MaxDesiredWidth(300.0f)
								[
									SAssignNew(SupportedCulturePicker, SCulturePicker)
									.OnSelectionChanged(this, &FLocalizationTargetDetailCustomization::OnNewSupportedCultureSelected)
									.IsCulturePickable(this, &FLocalizationTargetDetailCustomization::IsCultureSelectableAsSupported)
								]
							]
					]
			];
	});

	{
		// The sort priority is set the first time we edit the category, so set it here first
		IDetailCategoryBuilder& DetailCategoryBuilder = DetailBuilder.EditCategory("Target", LOCTEXT("TargetCategoryLabel","Target"), ECategoryPriority::Variable);
	}

	// We need to add the customizations in the same order as the properties to ensure that things are ordered correctly
	FStructProperty* const SettingsStructProperty = CastFieldChecked<FStructProperty>(TargetSettingsPropertyHandle->GetProperty());
	for (TFieldIterator<FProperty> Iterator(SettingsStructProperty->Struct); Iterator; ++Iterator)
	{
		FProperty* const MemberProperty = *Iterator;
		
		if (!MemberProperty->HasAnyPropertyFlags(CPF_Edit))
		{
			continue;
		}
		
		const FName PropertyName = MemberProperty->GetFName();
		const TSharedPtr<IPropertyHandle> MemberPropertyHandle = TargetSettingsPropertyHandle->GetChildHandle(PropertyName);
		if (MemberPropertyHandle.IsValid() && MemberPropertyHandle->IsValidHandle())
		{
			static const FName ShowOnlyInners("ShowOnlyInnerProperties");

			const FName CategoryName = FObjectEditorUtils::GetCategoryFName(MemberProperty);
			IDetailCategoryBuilder& DetailCategoryBuilder = DetailBuilder.EditCategory(CategoryName);

			const auto* const Function = PropertyCustomizationMap.Find(PropertyName);
			if (Function)
			{
				MemberPropertyHandle->MarkHiddenByCustomization();
				(*Function)(MemberPropertyHandle.ToSharedRef(), DetailCategoryBuilder);
			}
			else if (MemberPropertyHandle->HasMetaData(ShowOnlyInners))
			{
				// This property is marked as ShowOnlyInnerProperties, so hoist its child properties up-to this level
				MemberPropertyHandle->MarkHiddenByCustomization();

				uint32 NumChildProperties = 0;
				MemberPropertyHandle->GetNumChildren(NumChildProperties);

				for (uint32 ChildPropertyIndex = 0; ChildPropertyIndex < NumChildProperties; ++ChildPropertyIndex)
				{
					const TSharedPtr<IPropertyHandle> ChildPropertyHandle = MemberPropertyHandle->GetChildHandle(ChildPropertyIndex);
					if (ChildPropertyHandle.IsValid() && ChildPropertyHandle->IsValidHandle())
					{
						DetailCategoryBuilder.AddProperty(ChildPropertyHandle);
					}
				}
			}
		}
	}

	{
		IDetailCategoryBuilder& DetailCategoryBuilder = DetailBuilder.EditCategory("Target");
		FDetailWidgetRow& DetailWidgetRow = DetailCategoryBuilder.AddCustomRow(LOCTEXT("LocalizationTargetLoadingPolicyRowFilterString", "Loading Policy"));

		static const TArray< TSharedPtr<ELocalizationTargetLoadingPolicy> > LoadingPolicies = []()
		{
			UEnum* const LoadingPolicyEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Localization.ELocalizationTargetLoadingPolicy"));
			TArray< TSharedPtr<ELocalizationTargetLoadingPolicy> > Array;
			for (int32 i = 0; i < LoadingPolicyEnum->NumEnums() - 1; ++i)
			{
				Array.Add( MakeShareable( new ELocalizationTargetLoadingPolicy(static_cast<ELocalizationTargetLoadingPolicy>(i)) ) );
			}
			return Array;
		}();

		DetailWidgetRow.NameContent()
			[
				SNew(STextBlock)
				.Font(DetailLayoutBuilder->GetDetailFont())
				.Text(LOCTEXT("LocalizationTargetLoadingPolicyRowName", "Loading Policy"))
			];
		DetailWidgetRow.ValueContent()
			[
				SNew(SComboBox< TSharedPtr<ELocalizationTargetLoadingPolicy> >)
				.OptionsSource(&LoadingPolicies)
				.OnSelectionChanged(this, &FLocalizationTargetDetailCustomization::OnLoadingPolicySelectionChanged)
				.OnGenerateWidget(this, &FLocalizationTargetDetailCustomization::GenerateWidgetForLoadingPolicy)
				.InitiallySelectedItem(LoadingPolicies[static_cast<int32>(GetLoadingPolicy())])
				.Content()
				[
					SNew(STextBlock)
					.Font(DetailLayoutBuilder->GetDetailFont())
					.Text_Lambda([this]() 
					{
						UEnum* const LoadingPolicyEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Localization.ELocalizationTargetLoadingPolicy"));
						return LoadingPolicyEnum->GetDisplayNameTextByValue(static_cast<int64>(GetLoadingPolicy()));
					})
				]
			];
	}
}

FLocalizationTargetSettings* FLocalizationTargetDetailCustomization::GetTargetSettings() const
{
	return LocalizationTarget.IsValid() ? &(LocalizationTarget->Settings) : nullptr;
}

TSharedPtr<IPropertyHandle> FLocalizationTargetDetailCustomization::GetTargetSettingsPropertyHandle() const
{
	return TargetSettingsPropertyHandle;
}

FText FLocalizationTargetDetailCustomization::GetTargetName() const
{
	if (LocalizationTarget.IsValid())
	{
		return FText::FromString(LocalizationTarget->Settings.Name);
	}
	return FText::GetEmpty();
}

bool FLocalizationTargetDetailCustomization::IsTargetNameUnique(const FString& Name) const
{
	TArray<ULocalizationTarget*> AllLocalizationTargets;
	ULocalizationTargetSet* EngineTargetSet = ULocalizationSettings::GetEngineTargetSet();
	if (EngineTargetSet != TargetSet)
	{
		AllLocalizationTargets.Append(EngineTargetSet->TargetObjects);
	}
	AllLocalizationTargets.Append(TargetSet->TargetObjects);

	for (ULocalizationTarget* const TargetObject : AllLocalizationTargets)
	{
		if (TargetObject != LocalizationTarget)
		{
			if (TargetObject->Settings.Name == LocalizationTarget->Settings.Name)
			{
				return false;
			}
		}
	}
	return true;
}

void FLocalizationTargetDetailCustomization::OnTargetNameChanged(const FText& NewText)
{
	const FString& NewName = NewText.ToString();

	// TODO: Target names must be valid directory names, because they are used as directory names.
	// ValidatePath allows /, which is not a valid directory name character
	FText Error;
	if (!FPaths::ValidatePath(NewName, &Error))
	{
		TargetNameEditableTextBox->SetError(Error);
		return;		
	}

	// Target name must be unique.
	if (!IsTargetNameUnique(NewName))
	{
		TargetNameEditableTextBox->SetError(LOCTEXT("DuplicateTargetNameError", "Target name must be unique."));
		return;
	}

	// Clear error if nothing has failed.
	TargetNameEditableTextBox->SetError(FText::GetEmpty());
}

void FLocalizationTargetDetailCustomization::OnTargetNameCommitted(const FText& NewText, ETextCommit::Type Type)
{
	// Target name must be unique.
	if (!IsTargetNameUnique(NewText.ToString()))
	{
		return;
	}

	if (TargetSettingsPropertyHandle.IsValid() && TargetSettingsPropertyHandle->IsValidHandle())
	{
		FLocalizationTargetSettings* const TargetSettings = GetTargetSettings();
		if (TargetSettings)
		{
			// Early out if the committed name is the same as the current name.
			if (TargetSettings->Name == NewText.ToString())
			{
				return;
			}

			const TSharedPtr<IPropertyHandle> TargetNamePropertyHandle = TargetSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, Name));

			if (TargetNamePropertyHandle.IsValid() && TargetNamePropertyHandle->IsValidHandle())
			{
				TargetNamePropertyHandle->NotifyPreChange();
			}

			LocalizationTarget->RenameTargetAndFiles(NewText.ToString());

			if (TargetNamePropertyHandle.IsValid() && TargetNamePropertyHandle->IsValidHandle())
			{
				TargetNamePropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			}
		}
	}
}

ELocalizationTargetLoadingPolicy FLocalizationTargetDetailCustomization::GetLoadingPolicy() const
{
	const FString DataDirectory = LocalizationConfigurationScript::GetDataDirectory(LocalizationTarget.Get());

	for (const FLocalizationTargetLoadingPolicyConfig& LoadingPolicyConfig : LoadingPolicyConfigs)
	{
		TArray<FString> LocalizationPaths;
		GConfig->GetArray(*LoadingPolicyConfig.SectionName, *LoadingPolicyConfig.KeyName, LocalizationPaths, LoadingPolicyConfig.ConfigPath);

		if (LocalizationPaths.Contains(DataDirectory))
		{
			return LoadingPolicyConfig.LoadingPolicy;
		}
	}

	return ELocalizationTargetLoadingPolicy::Never;
}

void FLocalizationTargetDetailCustomization::SetLoadingPolicy(const ELocalizationTargetLoadingPolicy LoadingPolicy)
{
	const FString DataDirectory = LocalizationConfigurationScript::GetDataDirectory(LocalizationTarget.Get());
	const FString CollapsedDataDirectory = FConfigValue::CollapseValue(DataDirectory);

	enum class EDefaultConfigOperation : uint8
	{
		AddExclusion,
		RemoveExclusion,
		AddAddition,
		RemoveAddition,
	};

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	auto ProcessDefaultConfigOperation = [&](const FLocalizationTargetLoadingPolicyConfig& LoadingPolicyConfig, const EDefaultConfigOperation OperationToPerform)
	{
		// We test the coalesced config data first, as we may be inheriting this target path from a base config.
		TArray<FString> LocalizationPaths;
		GConfig->GetArray(*LoadingPolicyConfig.SectionName, *LoadingPolicyConfig.KeyName, LocalizationPaths, LoadingPolicyConfig.ConfigPath);
		const bool bHasTargetPath = LocalizationPaths.Contains(DataDirectory);

		// Work out whether we need to do work with the default config...
		switch (OperationToPerform)
		{
		case EDefaultConfigOperation::AddExclusion:
		case EDefaultConfigOperation::RemoveAddition:
			if (!bHasTargetPath)
			{
				return; // No point removing a target that doesn't exist
			}
			break;
		case EDefaultConfigOperation::AddAddition:
		case EDefaultConfigOperation::RemoveExclusion:
			if (bHasTargetPath)
			{
				return; // No point adding a target that already exists
			}
			break;
		default:
			break;
		}

		FConfigFile IniFile;
		FConfigCacheIni::LoadLocalIniFile(IniFile, *LoadingPolicyConfig.DefaultConfigName, /*bIsBaseIniName*/false);

		switch (OperationToPerform)
		{
		case EDefaultConfigOperation::AddExclusion:
			IniFile.AddToSection(*LoadingPolicyConfig.SectionName, *FString::Printf(TEXT("-%s"), *LoadingPolicyConfig.KeyName), *CollapsedDataDirectory);
			break;
		case EDefaultConfigOperation::RemoveExclusion:
			IniFile.RemoveFromSection(*LoadingPolicyConfig.SectionName, *FString::Printf(TEXT("-%s"), *LoadingPolicyConfig.KeyName), *CollapsedDataDirectory);
			break;
		case EDefaultConfigOperation::AddAddition:
			IniFile.AddToSection(*LoadingPolicyConfig.SectionName, *FString::Printf(TEXT("+%s"), *LoadingPolicyConfig.KeyName), *CollapsedDataDirectory);
			break;
		case EDefaultConfigOperation::RemoveAddition:
			IniFile.RemoveFromSection(*LoadingPolicyConfig.SectionName, *FString::Printf(TEXT("+%s"), *LoadingPolicyConfig.KeyName), *CollapsedDataDirectory);
			break;
		default:
			break;
		}

		// Make sure the file is checked out (if needed).
		if (SourceControlProvider.IsEnabled())
		{
			FSourceControlStatePtr ConfigFileState = SourceControlProvider.GetState(LoadingPolicyConfig.DefaultConfigFilePath, EStateCacheUsage::Use);
			if (!ConfigFileState.IsValid() || ConfigFileState->IsUnknown())
			{
				ConfigFileState = SourceControlProvider.GetState(LoadingPolicyConfig.DefaultConfigFilePath, EStateCacheUsage::ForceUpdate);
			}
			if (ConfigFileState.IsValid() && ConfigFileState->IsSourceControlled() && !(ConfigFileState->IsCheckedOut() || ConfigFileState->IsAdded()) && ConfigFileState->CanCheckout())
			{
				SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), LoadingPolicyConfig.DefaultConfigFilePath);
			}
		}
		else
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if (PlatformFile.FileExists(*LoadingPolicyConfig.DefaultConfigFilePath) && PlatformFile.IsReadOnly(*LoadingPolicyConfig.DefaultConfigFilePath))
			{
				PlatformFile.SetReadOnly(*LoadingPolicyConfig.DefaultConfigFilePath, false);
			}
		}

		// Write out the new config.
		IniFile.Dirty = true;
		IniFile.UpdateSections(*LoadingPolicyConfig.DefaultConfigFilePath);

		// Make sure to add the file now (if needed).
		if (SourceControlProvider.IsEnabled())
		{
			FSourceControlStatePtr ConfigFileState = SourceControlProvider.GetState(LoadingPolicyConfig.DefaultConfigFilePath, EStateCacheUsage::Use);
			if (ConfigFileState.IsValid() && !ConfigFileState->IsSourceControlled() && ConfigFileState->CanAdd())
			{
				SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), LoadingPolicyConfig.DefaultConfigFilePath);
			}
		}

		// Reload the updated file into the config system.
		FConfigContext::ForceReloadIntoGConfig().Load(*LoadingPolicyConfig.BaseConfigName);
	};

	for (const FLocalizationTargetLoadingPolicyConfig& LoadingPolicyConfig : LoadingPolicyConfigs)
	{
		if (LoadingPolicyConfig.LoadingPolicy == LoadingPolicy)
		{
			// We need to remove any exclusions for this path, and add the path if needed.
			ProcessDefaultConfigOperation(LoadingPolicyConfig, EDefaultConfigOperation::RemoveExclusion);
			ProcessDefaultConfigOperation(LoadingPolicyConfig, EDefaultConfigOperation::AddAddition);
		}
		else
		{
			// We need to remove any additions for this path, and exclude the path is needed.
			ProcessDefaultConfigOperation(LoadingPolicyConfig, EDefaultConfigOperation::RemoveAddition);
			ProcessDefaultConfigOperation(LoadingPolicyConfig, EDefaultConfigOperation::AddExclusion);
		}
	}
}

void FLocalizationTargetDetailCustomization::OnLoadingPolicySelectionChanged(TSharedPtr<ELocalizationTargetLoadingPolicy> LoadingPolicy, ESelectInfo::Type SelectInfo)
{
	SetLoadingPolicy(*LoadingPolicy.Get());
};

TSharedRef<SWidget> FLocalizationTargetDetailCustomization::GenerateWidgetForLoadingPolicy(TSharedPtr<ELocalizationTargetLoadingPolicy> LoadingPolicy)
{
	UEnum* const LoadingPolicyEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Localization.ELocalizationTargetLoadingPolicy"));
	return SNew(STextBlock)
		.Font(DetailLayoutBuilder->GetDetailFont())
		.Text(LoadingPolicyEnum->GetDisplayNameTextByValue(static_cast<int64>(*LoadingPolicy.Get())));
};

void FLocalizationTargetDetailCustomization::RebuildTargetDependenciesBox()
{
	if (TargetDependenciesHorizontalBox.IsValid())
	{
		for (const TSharedPtr<SWidget>& Widget : TargetDependenciesWidgets)
		{
			TargetDependenciesHorizontalBox->RemoveSlot(Widget.ToSharedRef());
		}
		TargetDependenciesWidgets.Empty();

		TArray<ULocalizationTarget*> AllLocalizationTargets;
		ULocalizationTargetSet* EngineTargetSet = ULocalizationSettings::GetEngineTargetSet();
		if (EngineTargetSet != TargetSet)
		{
			AllLocalizationTargets.Append(EngineTargetSet->TargetObjects);
		}
		AllLocalizationTargets.Append(TargetSet->TargetObjects);

		for (const FGuid& TargetDependencyGuid : LocalizationTarget->Settings.TargetDependencies)
		{
			ULocalizationTarget** const TargetDependency = AllLocalizationTargets.FindByPredicate([TargetDependencyGuid](ULocalizationTarget* SomeLocalizationTarget)->bool{return SomeLocalizationTarget->Settings.Guid == TargetDependencyGuid;});
			if (TargetDependency)
			{
				TWeakObjectPtr<ULocalizationTarget> TargetDependencyPtr = *TargetDependency;
				const auto& GetTargetDependencyName = [TargetDependencyPtr] {return FText::FromString(TargetDependencyPtr->Settings.Name);};
				const TSharedRef<SWidget> Widget = SNew(SBorder)
				[
					SNew(STextBlock)
					.Font(DetailLayoutBuilder->GetDetailFont())
					.Text_Lambda(GetTargetDependencyName)
				];

				TargetDependenciesWidgets.Add(Widget);
				TargetDependenciesHorizontalBox->AddSlot()
				[
					Widget
				];
			}
		}
	}
}

void FLocalizationTargetDetailCustomization::RebuildTargetsList()
{
	TargetDependenciesOptionsList.Empty();
	TFunction<bool (ULocalizationTarget* const)> DoesTargetDependOnUs;
	DoesTargetDependOnUs = [&, this](ULocalizationTarget* const OtherTarget) -> bool
	{
		if (OtherTarget->Settings.TargetDependencies.Contains(LocalizationTarget->Settings.Guid))
		{
			return true;
		}

		for (const FGuid& OtherTargetDependencyGuid : OtherTarget->Settings.TargetDependencies)
		{
			TObjectPtr<ULocalizationTarget>* const OtherTargetDependency = TargetSet->TargetObjects.FindByPredicate([OtherTargetDependencyGuid](ULocalizationTarget* SomeLocalizationTarget)->bool{return SomeLocalizationTarget->Settings.Guid == OtherTargetDependencyGuid;});
			if (OtherTargetDependency && DoesTargetDependOnUs(*OtherTargetDependency))
			{
				return true;
			}
		}

		return false;
	};

	TArray<ULocalizationTarget*> AllLocalizationTargets;
	ULocalizationTargetSet* EngineTargetSet = ULocalizationSettings::GetEngineTargetSet();
	if (EngineTargetSet != TargetSet)
	{
		AllLocalizationTargets.Append(EngineTargetSet->TargetObjects);
	}
	AllLocalizationTargets.Append(TargetSet->TargetObjects);
	for (ULocalizationTarget* const OtherTarget : AllLocalizationTargets)
	{
		if (OtherTarget != LocalizationTarget)
		{
			if (!DoesTargetDependOnUs(OtherTarget))
			{
				TargetDependenciesOptionsList.Add(OtherTarget);
			}
		}
	}

	if (TargetDependenciesListView.IsValid())
	{
		TargetDependenciesListView->RequestListRefresh();
	}
}

TSharedRef<ITableRow> FLocalizationTargetDetailCustomization::OnGenerateTargetRow(ULocalizationTarget* OtherLocalizationTarget, const TSharedRef<STableViewBase>& Table)
{
	return SNew(STableRow<ULocalizationTarget*>, Table)
		.ShowSelection(true)
		.Content()
		[
			SNew(SCheckBox)
			.OnCheckStateChanged_Lambda([this, OtherLocalizationTarget](ECheckBoxState State)
			{
				OnTargetDependencyCheckStateChanged(OtherLocalizationTarget, State);
			})
				.IsChecked_Lambda([this, OtherLocalizationTarget]()->ECheckBoxState
			{
				return IsTargetDependencyChecked(OtherLocalizationTarget);
			})
				.Content()
				[
					SNew(STextBlock)
					.Text(FText::FromString(OtherLocalizationTarget->Settings.Name))
				]
		];
}

void FLocalizationTargetDetailCustomization::OnTargetDependencyCheckStateChanged(ULocalizationTarget* const OtherLocalizationTarget, const ECheckBoxState State)
{
	const TSharedPtr<IPropertyHandle> TargetDependenciesPropertyHandle = TargetSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, TargetDependencies));

	if (TargetDependenciesPropertyHandle.IsValid() && TargetDependenciesPropertyHandle->IsValidHandle())
	{
		TargetDependenciesPropertyHandle->NotifyPreChange();
	}

	switch (State)
	{
	case ECheckBoxState::Checked:
		LocalizationTarget->Settings.TargetDependencies.Add(OtherLocalizationTarget->Settings.Guid);
		break;
	case ECheckBoxState::Unchecked:
		LocalizationTarget->Settings.TargetDependencies.Remove(OtherLocalizationTarget->Settings.Guid);
		break;
	}

	if (TargetDependenciesPropertyHandle.IsValid() && TargetDependenciesPropertyHandle->IsValidHandle())
	{
		TargetDependenciesPropertyHandle->NotifyPostChange(State == ECheckBoxState::Checked ? EPropertyChangeType::ArrayAdd : EPropertyChangeType::ArrayRemove);
	}

	RebuildTargetDependenciesBox();
}

ECheckBoxState FLocalizationTargetDetailCustomization::IsTargetDependencyChecked(ULocalizationTarget* const OtherLocalizationTarget) const
{
	return LocalizationTarget->Settings.TargetDependencies.Contains(OtherLocalizationTarget->Settings.Guid) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool FLocalizationTargetDetailCustomization::CanGatherText() const
{
	return LocalizationTarget.IsValid() && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

void FLocalizationTargetDetailCustomization::GatherText()
{
	if (LocalizationTarget.IsValid())
	{
		// Save unsaved packages.
		const bool bPromptUserToSave = true;
		const bool bSaveMapPackages = true;
		const bool bSaveContentPackages = true;
		const bool bFastSave = false;
		const bool bNotifyNoPackagesSaved = false;
		const bool bCanBeDeclined = true;
		bool DidPackagesNeedSaving;
		const bool WerePackagesSaved = FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined, &DidPackagesNeedSaving);

		if (DidPackagesNeedSaving && !WerePackagesSaved)
		{
			// Give warning dialog.
			const FText MessageText = NSLOCTEXT("LocalizationCultureActions", "UnsavedPackagesWarningDialogMessage", "There are unsaved changes. These changes may not be gathered from correctly.");
			const FText TitleText = NSLOCTEXT("LocalizationCultureActions", "UnsavedPackagesWarningDialogTitle", "Unsaved Changes Before Gather");
			switch(FMessageDialog::Open(EAppMsgType::OkCancel, MessageText, TitleText))
			{
			case EAppReturnType::Cancel:
				{
					return;
				}
			}
		}

		// Execute gather.
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
		LocalizationCommandletTasks::GatherTextForTarget(ParentWindow.ToSharedRef(), LocalizationTarget.Get());

		UpdateTargetFromReports();
	}
}

bool FLocalizationTargetDetailCustomization::CanImportTextAllCultures() const
{
	return LocalizationTarget.IsValid() && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

void FLocalizationTargetDetailCustomization::ImportTextAllCultures()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (LocalizationTarget.IsValid() && DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString DefaultPath = FPaths::ConvertRelativePathToFull(LocalizationConfigurationScript::GetDataDirectory(LocalizationTarget.Get()));

		FText DialogTitle;
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			DialogTitle = FText::Format(LOCTEXT("ImportAllTranslationsForTargetDialogTitleFormat", "Import All Translations for {TargetName} from Directory"), FormatArguments);
		}

		// Prompt the user for the directory
		FString OutputDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, DialogTitle.ToString(), DefaultPath, OutputDirectory))
		{
			LocalizationCommandletTasks::ImportTextForTarget(ParentWindow.ToSharedRef(), LocalizationTarget.Get(), TOptional<FString>(OutputDirectory));

			UpdateTargetFromReports();
		}
	}
}

bool FLocalizationTargetDetailCustomization::CanExportTextAllCultures() const
{
	return LocalizationTarget.IsValid() && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

void FLocalizationTargetDetailCustomization::ExportTextAllCultures()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (LocalizationTarget.IsValid() && DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString DefaultPath = FPaths::ConvertRelativePathToFull(LocalizationConfigurationScript::GetDataDirectory(LocalizationTarget.Get()));

		FText DialogTitle;
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			DialogTitle = FText::Format(LOCTEXT("ExportAllTranslationsForTargetDialogTitleFormat", "Export All Translations for {TargetName} to Directory"), FormatArguments);
		}

		// Prompt the user for the directory
		FString OutputDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, DialogTitle.ToString(), DefaultPath, OutputDirectory))
		{
			LocalizationCommandletTasks::ExportTextForTarget(ParentWindow.ToSharedRef(), LocalizationTarget.Get(), TOptional<FString>(OutputDirectory));
		}
	}
}

bool FLocalizationTargetDetailCustomization::CanImportDialogueScriptAllCultures() const
{
	return LocalizationTarget.IsValid() && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

void FLocalizationTargetDetailCustomization::ImportDialogueScriptAllCultures()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (LocalizationTarget.IsValid() && DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString DefaultPath = FPaths::ConvertRelativePathToFull(LocalizationConfigurationScript::GetDataDirectory(LocalizationTarget.Get()));

		FText DialogTitle;
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			DialogTitle = FText::Format(LOCTEXT("ImportAllDialogueScriptsForTargetDialogTitleFormat", "Import All Dialogue Scripts for {TargetName} from Directory"), FormatArguments);
		}

		// Prompt the user for the directory
		FString OutputDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, DialogTitle.ToString(), DefaultPath, OutputDirectory))
		{
			LocalizationCommandletTasks::ImportDialogueScriptForTarget(ParentWindow.ToSharedRef(), LocalizationTarget.Get(), TOptional<FString>(OutputDirectory));

			UpdateTargetFromReports();
		}
	}
}

bool FLocalizationTargetDetailCustomization::CanExportDialogueScriptAllCultures() const
{
	return LocalizationTarget.IsValid() && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

void FLocalizationTargetDetailCustomization::ExportDialogueScriptAllCultures()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (LocalizationTarget.IsValid() && DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString DefaultPath = FPaths::ConvertRelativePathToFull(LocalizationConfigurationScript::GetDataDirectory(LocalizationTarget.Get()));

		FText DialogTitle;
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			DialogTitle = FText::Format(LOCTEXT("ExportAllDialogueScriptsForTargetDialogTitleFormat", "Export All Dialogue Scripts for {TargetName} to Directory"), FormatArguments);
		}

		// Prompt the user for the directory
		FString OutputDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, DialogTitle.ToString(), DefaultPath, OutputDirectory))
		{
			LocalizationCommandletTasks::ExportDialogueScriptForTarget(ParentWindow.ToSharedRef(), LocalizationTarget.Get(), TOptional<FString>(OutputDirectory));
		}
	}
}

bool FLocalizationTargetDetailCustomization::CanImportDialogueAllCultures() const
{
	return LocalizationTarget.IsValid() && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

void FLocalizationTargetDetailCustomization::ImportDialogueAllCultures()
{
	if (LocalizationTarget.IsValid())
	{
		// Warn about potentially loaded audio assets
		{
			TArray<ULocalizationTarget*> Targets;
			Targets.Add(LocalizationTarget.Get());

			if (!LocalizationCommandletTasks::ReportLoadedAudioAssets(Targets))
			{
				return;
			}
		}

		// Execute import dialogue.
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
		LocalizationCommandletTasks::ImportDialogueForTarget(ParentWindow.ToSharedRef(), LocalizationTarget.Get());
	}
}

bool FLocalizationTargetDetailCustomization::CanCountWords() const
{
	return LocalizationTarget.IsValid() && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

void FLocalizationTargetDetailCustomization::CountWords()
{
	if (LocalizationTarget.IsValid())
	{
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
		LocalizationCommandletTasks::GenerateWordCountReportForTarget(ParentWindow.ToSharedRef(), LocalizationTarget.Get());

		UpdateTargetFromReports();
	}
}

bool FLocalizationTargetDetailCustomization::CanCompileTextAllCultures() const
{
	return LocalizationTarget.IsValid() && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

void FLocalizationTargetDetailCustomization::CompileTextAllCultures()
{
	if (LocalizationTarget.IsValid())
	{
		// Execute compile.
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
		LocalizationCommandletTasks::CompileTextForTarget(ParentWindow.ToSharedRef(), LocalizationTarget.Get());
	}
}

void FLocalizationTargetDetailCustomization::UpdateTargetFromReports()
{
	if (LocalizationTarget.IsValid())
	{
		TArray< TSharedPtr<IPropertyHandle> > WordCountPropertyHandles;

		if (TargetSettingsPropertyHandle.IsValid() && TargetSettingsPropertyHandle->IsValidHandle())
		{
			const TSharedPtr<IPropertyHandle> SupportedCulturesStatisticsPropHandle = TargetSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, SupportedCulturesStatistics));
			if (SupportedCulturesStatisticsPropHandle.IsValid() && SupportedCulturesStatisticsPropHandle->IsValidHandle())
			{
				uint32 SupportedCultureCount = 0;
				SupportedCulturesStatisticsPropHandle->GetNumChildren(SupportedCultureCount);
				for (uint32 i = 0; i < SupportedCultureCount; ++i)
				{
					const TSharedPtr<IPropertyHandle> ElementPropertyHandle = SupportedCulturesStatisticsPropHandle->GetChildHandle(i);
					if (ElementPropertyHandle.IsValid() && ElementPropertyHandle->IsValidHandle())
					{
						const TSharedPtr<IPropertyHandle> WordCountPropertyHandle = SupportedCulturesStatisticsPropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCultureStatistics, WordCount));
						if (WordCountPropertyHandle.IsValid() && WordCountPropertyHandle->IsValidHandle())
						{
							WordCountPropertyHandles.Add(WordCountPropertyHandle);
						}
					}
				}
			}
		}

		for (const TSharedPtr<IPropertyHandle>& WordCountPropertyHandle : WordCountPropertyHandles)
		{
			WordCountPropertyHandle->NotifyPreChange();
		}
		LocalizationTarget->UpdateWordCountsFromCSV();
		LocalizationTarget->UpdateStatusFromConflictReport();
		for (const TSharedPtr<IPropertyHandle>& WordCountPropertyHandle : WordCountPropertyHandles)
		{
			WordCountPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
	}
}

void FLocalizationTargetDetailCustomization::BuildListedCulturesList()
{
	const TSharedPtr<IPropertyHandleArray> SupportedCulturesStatisticsArrayPropertyHandle = SupportedCulturesStatisticsPropertyHandle->AsArray();
	if (SupportedCulturesStatisticsArrayPropertyHandle.IsValid())
	{
		uint32 ElementCount = 0;
		SupportedCulturesStatisticsArrayPropertyHandle->GetNumElements(ElementCount);
		for (uint32 i = 0; i < ElementCount; ++i)
		{
			const TSharedPtr<IPropertyHandle> CultureStatisticsProperty = SupportedCulturesStatisticsArrayPropertyHandle->GetElement(i);
			ListedCultureStatisticProperties.AddUnique(CultureStatisticsProperty);
		}
	}

	const auto& CultureSorter = [](const TSharedPtr<IPropertyHandle>& Left, const TSharedPtr<IPropertyHandle>& Right) -> bool
	{
		const TSharedPtr<IPropertyHandle> LeftNameHandle = Left->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCultureStatistics, CultureName));
		const TSharedPtr<IPropertyHandle> RightNameHandle = Right->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCultureStatistics, CultureName));

		FString LeftName;
		LeftNameHandle->GetValue(LeftName);
		const FCulturePtr LeftCulture = FInternationalization::Get().GetCulture(LeftName);
		FString RightName;
		RightNameHandle->GetValue(RightName);
		const FCulturePtr RightCulture = FInternationalization::Get().GetCulture(RightName);

		return LeftCulture.IsValid() && RightCulture.IsValid() ? LeftCulture->GetDisplayName() < RightCulture->GetDisplayName() : LeftName < RightName;
	};
	ListedCultureStatisticProperties.Sort(CultureSorter);

	if (NoSupportedCulturesErrorText.IsValid())
	{
		if (ListedCultureStatisticProperties.Num())
		{
			NoSupportedCulturesErrorText->SetError(FText::GetEmpty());
		}
		else
		{
			NoSupportedCulturesErrorText->SetError(LOCTEXT("NoSupportedCulturesError", "At least one supported language must be specified."));
		}
	}
}

void FLocalizationTargetDetailCustomization::RebuildListedCulturesList()
{
	if (NewEntryIndexToBeInitialized != INDEX_NONE)
	{
		const TSharedPtr<IPropertyHandle> SupportedCultureStatisticsPropertyHandle = SupportedCulturesStatisticsPropertyHandle->GetChildHandle(NewEntryIndexToBeInitialized);

		const TSharedPtr<IPropertyHandle> CultureNamePropertyHandle = SupportedCultureStatisticsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCultureStatistics, CultureName));
		if(CultureNamePropertyHandle.IsValid() && CultureNamePropertyHandle->IsValidHandle())
		{
			CultureNamePropertyHandle->SetValue(SelectedNewCulture->GetName());
		}

		const TSharedPtr<IPropertyHandle> WordCountPropertyHandle = SupportedCultureStatisticsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCultureStatistics, WordCount));
		if(WordCountPropertyHandle.IsValid() && WordCountPropertyHandle->IsValidHandle())
		{
			WordCountPropertyHandle->SetValue(0);
		}

		AddNewSupportedCultureComboButton->SetIsOpen(false);

		NewEntryIndexToBeInitialized = INDEX_NONE;
	}

	ListedCultureStatisticProperties.Empty();
	BuildListedCulturesList();

	if (SupportedCultureListView.IsValid())
	{
		SupportedCultureListView->RequestListRefresh();
	}
}

TSharedRef<ITableRow> FLocalizationTargetDetailCustomization::OnGenerateCultureRow(TSharedPtr<IPropertyHandle> CultureStatisticsPropertyHandle, const TSharedRef<STableViewBase>& Table)
{
	return SNew(SLocalizationTargetEditorCultureRow, Table, DetailLayoutBuilder->GetPropertyUtilities(), TargetSettingsPropertyHandle.ToSharedRef(), CultureStatisticsPropertyHandle->GetIndexInArray());
}

bool FLocalizationTargetDetailCustomization::IsCultureSelectableAsSupported(FCulturePtr Culture)
{
	auto Is = [&](const TSharedPtr<IPropertyHandle>& SupportedCultureStatisticProperty)
	{
		// Can't select existing supported cultures.
		const TSharedPtr<IPropertyHandle> SupportedCultureNamePropertyHandle = SupportedCultureStatisticProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCultureStatistics, CultureName));
		if (SupportedCultureNamePropertyHandle.IsValid() && SupportedCultureNamePropertyHandle->IsValidHandle())
		{
			FString SupportedCultureName;
			SupportedCultureNamePropertyHandle->GetValue(SupportedCultureName);
			const FCulturePtr SupportedCulture = FInternationalization::Get().GetCulture(SupportedCultureName);
			return SupportedCulture == Culture;
		}

		return false;
	};

	return !ListedCultureStatisticProperties.ContainsByPredicate(Is);
}

void FLocalizationTargetDetailCustomization::OnNewSupportedCultureSelected(FCulturePtr SelectedCulture, ESelectInfo::Type SelectInfo)
{
	if(SupportedCulturesStatisticsPropertyHandle.IsValid() && SupportedCulturesStatisticsPropertyHandle->IsValidHandle())
	{
		uint32 NewElementIndex;
		SupportedCulturesStatisticsPropertyHandle->AsArray()->GetNumElements(NewElementIndex);

		// Add element, set info for later initialization.
		SelectedNewCulture = SelectedCulture;
		NewEntryIndexToBeInitialized = NewElementIndex;
		SupportedCulturesStatisticsPropertyHandle->AsArray()->AddItem();

		if (NativeCultureIndexPropertyHandle.IsValid() && NativeCultureIndexPropertyHandle->IsValidHandle())
		{
			int32 NativeCultureIndex;
			NativeCultureIndexPropertyHandle->GetValue(NativeCultureIndex);
			if (NativeCultureIndex == INDEX_NONE)
			{
				NativeCultureIndex = NewElementIndex;
				NativeCultureIndexPropertyHandle->SetValue(NativeCultureIndex);
			}
		}

		// Refresh UI.
		if (SupportedCulturePicker.IsValid())
		{
			SupportedCulturePicker->RequestTreeRefresh();
		}
	}
}

#undef LOCTEXT_NAMESPACE
