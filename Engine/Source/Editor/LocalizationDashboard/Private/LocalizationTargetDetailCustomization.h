// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailCustomization.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/Text.h"
#include "LocalizationTargetTypes.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Views/SListView.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class ITableRow;
class SComboButton;
class SCulturePicker;
class SEditableTextBox;
class SErrorText;
class SHorizontalBox;
class STableViewBase;
class SWidget;

enum class ELocalizationTargetLoadingPolicy : uint8;

class FLocalizationTargetDetailCustomization : public IDetailCustomization
{
public:
	FLocalizationTargetDetailCustomization();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	FLocalizationTargetSettings* GetTargetSettings() const;
	TSharedPtr<IPropertyHandle> GetTargetSettingsPropertyHandle() const;

private:
	FText GetTargetName() const;
	bool IsTargetNameUnique(const FString& Name) const;
	void OnTargetNameChanged(const FText& NewText);
	void OnTargetNameCommitted(const FText& NewText, ETextCommit::Type Type);

	ELocalizationTargetLoadingPolicy GetLoadingPolicy() const;
	void SetLoadingPolicy(const ELocalizationTargetLoadingPolicy LoadingPolicy);
	void OnLoadingPolicySelectionChanged(TSharedPtr<ELocalizationTargetLoadingPolicy> LoadingPolicy, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> GenerateWidgetForLoadingPolicy(TSharedPtr<ELocalizationTargetLoadingPolicy> LoadingPolicy);

	void RebuildTargetDependenciesBox();
	void RebuildTargetsList();
	TSharedRef<ITableRow> OnGenerateTargetRow(ULocalizationTarget* OtherLocalizationTarget, const TSharedRef<STableViewBase>& Table);
	void OnTargetDependencyCheckStateChanged(ULocalizationTarget* const OtherLocalizationTarget, const ECheckBoxState State);
	ECheckBoxState IsTargetDependencyChecked(ULocalizationTarget* const OtherLocalizationTarget) const;

	bool CanGatherText() const;
	void GatherText();

	bool CanImportTextAllCultures() const;
	void ImportTextAllCultures();
	
	bool CanExportTextAllCultures() const;
	void ExportTextAllCultures();

	bool CanImportDialogueScriptAllCultures() const;
	void ImportDialogueScriptAllCultures();

	bool CanExportDialogueScriptAllCultures() const;
	void ExportDialogueScriptAllCultures();
	
	bool CanImportDialogueAllCultures() const;
	void ImportDialogueAllCultures();

	bool CanCountWords() const;
	void CountWords();
	
	bool CanCompileTextAllCultures() const;
	void CompileTextAllCultures();
	
	void UpdateTargetFromReports();

	void BuildListedCulturesList();
	void RebuildListedCulturesList();
	TSharedRef<ITableRow> OnGenerateCultureRow(TSharedPtr<IPropertyHandle> CulturePropertyHandle, const TSharedRef<STableViewBase>& Table);

	bool IsCultureSelectableAsSupported(FCulturePtr Culture);
	void OnNewSupportedCultureSelected(FCulturePtr SelectedCulture, ESelectInfo::Type SelectInfo);

private:
	IDetailLayoutBuilder* DetailLayoutBuilder;

	TWeakObjectPtr<ULocalizationTargetSet> TargetSet;
	TWeakObjectPtr<ULocalizationTarget> LocalizationTarget;

	TSharedPtr<IPropertyHandle> TargetSettingsPropertyHandle;

	TSharedPtr<SEditableTextBox> TargetNameEditableTextBox;

	TSharedPtr<SHorizontalBox> TargetDependenciesHorizontalBox;
	TArray< TSharedPtr<SWidget> > TargetDependenciesWidgets;
	TArray<ULocalizationTarget*> TargetDependenciesOptionsList;
	TSharedPtr< SListView<ULocalizationTarget*> > TargetDependenciesListView;

	TArray<FCulturePtr> AllCultures;

	TSharedPtr<IPropertyHandle> NativeCultureIndexPropertyHandle;
	TSharedPtr<IPropertyHandle> SupportedCulturesStatisticsPropertyHandle;
	FSimpleDelegate SupportedCulturesStatisticsPropertyHandle_OnNumElementsChanged;
	TSharedPtr< SListView< TSharedPtr<IPropertyHandle> > > SupportedCultureListView;
	TSharedPtr<SErrorText> NoSupportedCulturesErrorText;
	TSharedPtr<SComboButton> AddNewSupportedCultureComboButton;
	TSharedPtr<SCulturePicker> SupportedCulturePicker;
	TArray< TSharedPtr<IPropertyHandle> > ListedCultureStatisticProperties;

	/* If set, the entry at the index specified needs to be initialized as soon as possible. */
	int32 NewEntryIndexToBeInitialized;
	FCulturePtr SelectedNewCulture;
};
