// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationTargetSetDetailCustomization.h"

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "DesktopPlatformModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "FileHelpers.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Views/ITypedTableView.h"
#include "GenericPlatform/GenericWindow.h"
#include "HAL/PlatformMisc.h"
#include "IDesktopPlatform.h"
#include "IDetailsView.h"
#include "ILocalizationServiceModule.h"
#include "ILocalizationServiceProvider.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "LocalizationCommandletTasks.h"
#include "LocalizationSettings.h"
#include "LocalizationTargetTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "ObjectEditorUtils.h"
#include "PropertyHandle.h"
#include "SLocalizationDashboardTargetRow.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class UObject;

#define LOCTEXT_NAMESPACE "LocalizationDashboard"

FLocalizationTargetSetDetailCustomization::FLocalizationTargetSetDetailCustomization()
	: DetailLayoutBuilder(nullptr)
	, NewEntryIndexToBeInitialized(INDEX_NONE)
{
}

namespace
{
	class FLocalizationDashboardCommands : public TCommands<FLocalizationDashboardCommands>
	{
	public:
		FLocalizationDashboardCommands() 
			: TCommands<FLocalizationDashboardCommands>("LocalizationDashboard", NSLOCTEXT("Contexts", "LocalizationDashboard", "Localization Dashboard"), NAME_None, FAppStyle::GetAppStyleSetName())
		{
		}

		TSharedPtr<FUICommandInfo> GatherTextAllTargets;
		TSharedPtr<FUICommandInfo> ImportTextAllTargets;
		TSharedPtr<FUICommandInfo> ExportTextAllTargets;
		TSharedPtr<FUICommandInfo> ImportDialogueScriptAllTargets;
		TSharedPtr<FUICommandInfo> ExportDialogueScriptAllTargets;
		TSharedPtr<FUICommandInfo> ImportDialogueAllTargets;
		TSharedPtr<FUICommandInfo> CountWordsForAllTargets;
		TSharedPtr<FUICommandInfo> CompileTextAllTargets;

		/** Initialize commands */
		virtual void RegisterCommands() override;
	};

	void FLocalizationDashboardCommands::RegisterCommands() 
	{
		UI_COMMAND(GatherTextAllTargets, "Gather Text", "Gather text for all targets in the project.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ImportTextAllTargets, "Import Text", "Import translations for all cultures of all targets in the project.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ExportTextAllTargets, "Export Text", "Export translations for all cultures of all targets in the project.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ImportDialogueScriptAllTargets, "Import Script", "Import dialogue scripts for all cultures of all targets in the project.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ExportDialogueScriptAllTargets, "Export Script", "Export dialogue scripts for all cultures of all targets in the project.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ImportDialogueAllTargets, "Import Dialogue", "Import dialogue WAV files for all cultures of all targets in the project.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(CountWordsForAllTargets, "Count Words", "Count translations for all targets in the project.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(CompileTextAllTargets, "Compile Text", "Compile translations for all targets in the project.", EUserInterfaceActionType::Button, FInputChord());
	}
}

void FLocalizationTargetSetDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailLayoutBuilder = &DetailBuilder;
	{
		TArray< TWeakObjectPtr<UObject> > ObjectsBeingCustomized;
		DetailLayoutBuilder->GetObjectsBeingCustomized(ObjectsBeingCustomized);
		TargetSet = CastChecked<ULocalizationTargetSet>(ObjectsBeingCustomized.Top().Get());
	}

	{
		const ILocalizationServiceProvider& LSP = ILocalizationServiceModule::Get().GetProvider();
		TargetObjectsPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULocalizationTargetSet,TargetObjects));
		if (TargetObjectsPropertyHandle->IsValidHandle())
		{
			FText CategoryDisplayName;
			bool bCollapseCategory = false;
			if (TargetSet == ULocalizationSettings::GetEngineTargetSet())
			{
				CategoryDisplayName = LOCTEXT("EngineTargetsCategoryDisplayName", "Engine Targets");
				bCollapseCategory = true;
			}
			else if (TargetSet == ULocalizationSettings::GetGameTargetSet())
			{
				CategoryDisplayName = LOCTEXT("GameTargetsCategoryDisplayName", "Game Targets");
			}

			const FName CategoryName = FObjectEditorUtils::GetCategoryFName(TargetObjectsPropertyHandle->GetProperty());
			IDetailCategoryBuilder& DetailCategoryBuilder = DetailBuilder.EditCategory(CategoryName, CategoryDisplayName);
			DetailCategoryBuilder.InitiallyCollapsed(bCollapseCategory);
			DetailCategoryBuilder.RestoreExpansionState(false); // disable this for now as both the game and engine targets share the same state variable

			TargetObjectsPropertyHandle->MarkHiddenByCustomization();
			TargetsArrayPropertyHandle_OnNumElementsChanged = FSimpleDelegate::CreateSP(this, &FLocalizationTargetSetDetailCustomization::RebuildTargetsList);
			TargetObjectsPropertyHandle->AsArray()->SetOnNumElementsChanged(TargetsArrayPropertyHandle_OnNumElementsChanged);

			FLocalizationDashboardCommands::Register();
			const TSharedRef< FUICommandList > CommandList = MakeShareable(new FUICommandList);

			// Let the localization service extend this toolbar
			TSharedRef<FExtender> LocalizationServiceExtender = MakeShareable(new FExtender);
#if LOCALIZATION_SERVICES_WITH_SLATE
			if (TargetSet.IsValid() && ILocalizationServiceModule::Get().IsEnabled())
			{
				LSP.CustomizeTargetSetToolbar(LocalizationServiceExtender, TargetSet);
			}
#endif

			FToolBarBuilder ToolBarBuilder( CommandList, FMultiBoxCustomization::AllowCustomization("LocalizationDashboard"), LocalizationServiceExtender, /*bForceSmallIcons*/true );

			TAttribute<FText> GatherAllTargetsToolTipTextAttribute = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([this]() -> FText
			{
				return CanGatherTextAllTargets() ? FLocalizationDashboardCommands::Get().GatherTextAllTargets->GetDescription() : LOCTEXT("GatherAllTargetsDisabledToolTip", "At least one target must have a native culture specified in order to gather.");
			}));
			CommandList->MapAction(FLocalizationDashboardCommands::Get().GatherTextAllTargets, FExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::GatherTextAllTargets), FCanExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::CanGatherTextAllTargets));
			ToolBarBuilder.AddToolBarButton(FLocalizationDashboardCommands::Get().GatherTextAllTargets, NAME_None, TAttribute<FText>(), GatherAllTargetsToolTipTextAttribute, FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationDashboard.GatherTextAllTargets"));

			CommandList->MapAction(FLocalizationDashboardCommands::Get().ImportTextAllTargets, FExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::ImportTextAllTargets), FCanExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::CanImportTextAllTargets));
			ToolBarBuilder.AddToolBarButton(FLocalizationDashboardCommands::Get().ImportTextAllTargets, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationDashboard.ImportTextAllTargetsAllCultures"));

			CommandList->MapAction(FLocalizationDashboardCommands::Get().ExportTextAllTargets, FExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::ExportTextAllTargets), FCanExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::CanExportTextAllTargets));
			ToolBarBuilder.AddToolBarButton(FLocalizationDashboardCommands::Get().ExportTextAllTargets, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationDashboard.ExportTextAllTargetsAllCultures"));

			CommandList->MapAction(FLocalizationDashboardCommands::Get().ImportDialogueScriptAllTargets, FExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::ImportDialogueScriptAllTargets), FCanExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::CanImportDialogueScriptAllTargets));
			ToolBarBuilder.AddToolBarButton(FLocalizationDashboardCommands::Get().ImportDialogueScriptAllTargets, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationDashboard.ImportDialogueScriptAllTargetsAllCultures"));

			CommandList->MapAction(FLocalizationDashboardCommands::Get().ExportDialogueScriptAllTargets, FExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::ExportDialogueScriptAllTargets), FCanExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::CanExportDialogueScriptAllTargets));
			ToolBarBuilder.AddToolBarButton(FLocalizationDashboardCommands::Get().ExportDialogueScriptAllTargets, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationDashboard.ExportDialogueScriptAllTargetsAllCultures"));

			CommandList->MapAction(FLocalizationDashboardCommands::Get().ImportDialogueAllTargets, FExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::ImportDialogueAllTargets), FCanExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::CanImportDialogueAllTargets));
			ToolBarBuilder.AddToolBarButton(FLocalizationDashboardCommands::Get().ImportDialogueAllTargets, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationDashboard.ImportDialogueAllTargetsAllCultures"));

			CommandList->MapAction(FLocalizationDashboardCommands::Get().CountWordsForAllTargets, FExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::CountWordsForAllTargets), FCanExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::CanCountWordsForAllTargets));
			ToolBarBuilder.AddToolBarButton(FLocalizationDashboardCommands::Get().CountWordsForAllTargets, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationDashboard.CountWordsForAllTargets"));

			CommandList->MapAction(FLocalizationDashboardCommands::Get().CompileTextAllTargets, FExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::CompileTextAllTargets), FCanExecuteAction::CreateSP(this, &FLocalizationTargetSetDetailCustomization::CanCompileTextAllTargets));
			ToolBarBuilder.AddToolBarButton(FLocalizationDashboardCommands::Get().CompileTextAllTargets, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LocalizationDashboard.CompileTextAllTargetsAllCultures"));

			if (ILocalizationServiceModule::Get().IsEnabled())
			{
				ToolBarBuilder.BeginSection("LocalizationService");
				ToolBarBuilder.EndSection();
			}

			BuildTargetsList();

			DetailCategoryBuilder.AddCustomRow(TargetObjectsPropertyHandle->GetPropertyDisplayName())
				.WholeRowContent()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						ToolBarBuilder.MakeWidget()
					]
					+SVerticalBox::Slot()
						.AutoHeight()
						[
							SAssignNew(TargetsListView, SListView< TSharedPtr<IPropertyHandle> >)
							.OnGenerateRow(this, &FLocalizationTargetSetDetailCustomization::OnGenerateRow)
							.ListItemsSource(&TargetsList)
							.SelectionMode(ESelectionMode::None)
							.HeaderRow
							(
							SNew(SHeaderRow)
							+SHeaderRow::Column("Target")
							.DefaultLabel( LOCTEXT("TargetColumnLabel", "Target"))
							.HAlignHeader(HAlign_Left)
							.HAlignCell(HAlign_Left)
							.VAlignCell(VAlign_Center)
							+SHeaderRow::Column("Status")
							.DefaultLabel( LOCTEXT("StatusColumnLabel", "Conflict Status"))
							.HAlignHeader(HAlign_Center)
							.HAlignCell(HAlign_Center)
							.VAlignCell(VAlign_Center)
							+SHeaderRow::Column("WordCount")
							.DefaultLabel( LOCTEXT("WordCountColumnLabel", "Word Count"))
							.HAlignHeader(HAlign_Center)
							.HAlignCell(HAlign_Center)
							.VAlignCell(VAlign_Center)
							+SHeaderRow::Column("Actions")
							.DefaultLabel(FText::GetEmpty())
							.HAlignHeader(HAlign_Right)
							.HAlignCell(HAlign_Right)
							.VAlignCell(VAlign_Center)
							)
						]
					+SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						[
							SNew(SButton)
							.Text(LOCTEXT("AddNewTargetButtonLabel", "Add New Target"))
							.OnClicked(this, &FLocalizationTargetSetDetailCustomization::OnNewTargetButtonClicked)
						]
				];
		}
	}
}

void FLocalizationTargetSetDetailCustomization::BuildTargetsList()
{
	const TSharedPtr<IPropertyHandleArray> TargetObjectsArrayPropertyHandle = TargetObjectsPropertyHandle->AsArray();
	if (TargetObjectsArrayPropertyHandle.IsValid())
	{
		uint32 ElementCount = 0;
		TargetObjectsArrayPropertyHandle->GetNumElements(ElementCount);
		for (uint32 i = 0; i < ElementCount; ++i)
		{
			const TSharedPtr<IPropertyHandle> TargetObjectPropertyHandle = TargetObjectsArrayPropertyHandle->GetElement(i);
			TargetsList.Add(TargetObjectPropertyHandle);
		}
	}
}

void FLocalizationTargetSetDetailCustomization::RebuildTargetsList()
{
	const TSharedPtr<IPropertyHandleArray> TargetObjectsArrayPropertyHandle = TargetObjectsPropertyHandle->AsArray();
	if (TargetObjectsArrayPropertyHandle.IsValid() && NewEntryIndexToBeInitialized != INDEX_NONE)
	{
		const TSharedPtr<IPropertyHandle> TargetObjectPropertyHandle = TargetObjectsArrayPropertyHandle->GetElement(NewEntryIndexToBeInitialized);
		if (TargetObjectPropertyHandle.IsValid() && TargetObjectPropertyHandle->IsValidHandle())
		{
			ULocalizationTarget* const NewTarget = NewObject<ULocalizationTarget>(TargetSet.Get());

			TArray<void*> RawData;
			TargetObjectsPropertyHandle->AccessRawData(RawData);
			void* RawDatum = RawData.Top();
			TArray<ULocalizationTarget*>& TargetObjectsArray = *(reinterpret_cast< TArray<ULocalizationTarget*>* >(RawDatum));
			FString NewTargetName = "NewTarget";
			auto TargetNameIsIdentical = [&](ULocalizationTarget* Target) -> bool
			{
				return (Target != NewTarget) && Target && (Target->Settings.Name == NewTargetName);
			};

			for (uint32 i = 0; TargetObjectsArray.ContainsByPredicate(TargetNameIsIdentical); ++i)
			{
				NewTargetName = FString::Printf(TEXT("NewTarget%u"), i);
			}

			NewTarget->Settings.Name = NewTargetName;
			const int32 NativeCultureIndex = NewTarget->Settings.SupportedCulturesStatistics.Add( FCultureStatistics(FInternationalization::Get().GetCurrentCulture()->GetName()) );
			NewTarget->Settings.NativeCultureIndex = NativeCultureIndex;

			TargetObjectPropertyHandle->SetValue(NewTarget);

			NewEntryIndexToBeInitialized = INDEX_NONE;
		}
	}

	TargetsList.Empty();
	BuildTargetsList();

	if (TargetsListView.IsValid())
	{
		TargetsListView->RequestListRefresh();
	}
}

bool FLocalizationTargetSetDetailCustomization::CanGatherTextAllTargets() const
{
	for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
	{
		// If any target can be gathered, then gathering all can be executed.
		if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
		{
			return true;
		}
	}

	return false;
}

void FLocalizationTargetSetDetailCustomization::GatherTextAllTargets()
{
	// Save unsaved packages.
	const bool bPromptUserToSave = true;
	const bool bSaveMapPackages = true;
	const bool bSaveContentPackages = true;
	const bool bFastSave = false;
	const bool bNotifyNoPackagesSaved = false;
	const bool bCanBeDeclined = true;
	bool DidPackagesNeedSaving;
	const bool WerePackagesSaved = FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined, &DidPackagesNeedSaving);

	if (DidPackagesNeedSaving && !WerePackagesSaved)
	{
		// Give warning dialog.
		const FText MessageText = NSLOCTEXT("LocalizationCultureActions", "UnsavedPackagesWarningDialogMessage", "There are unsaved changes. These changes may not be gathered from correctly.");
		const FText TitleText = NSLOCTEXT("LocalizationCultureActions", "UnsavedPackagesWarningDialogTitle", "Unsaved Changes Before Gather");
		switch(FMessageDialog::Open(EAppMsgType::OkCancel, MessageText, TitleText))
		{
		case EAppReturnType::Cancel:
			{
				return;
			}
		}
	}

	TArray<ULocalizationTarget*> TargetObjectsToProcess;
	for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
	{
		if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0)
		{
			TargetObjectsToProcess.Add(LocalizationTarget);
		}
	}

	// Execute gather.
	const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
	LocalizationCommandletTasks::GatherTextForTargets(ParentWindow.ToSharedRef(), TargetObjectsToProcess);

	for (ULocalizationTarget* const LocalizationTarget : TargetObjectsToProcess)
	{
		UpdateTargetFromReports(LocalizationTarget);
	}
}

bool FLocalizationTargetSetDetailCustomization::CanImportTextAllTargets() const
{
	for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
	{
		// If any target can be imported, then importing all can be executed.
		if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
		{
			return true;
		}
	}

	return false;
}

void FLocalizationTargetSetDetailCustomization::ImportTextAllTargets()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}


		const FString DefaultPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir() / TEXT("Localization"));

		// Prompt the user for the directory
		FString OutputDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, LOCTEXT("ImportAllTranslationsDialogTitle", "Import All Translations from Directory").ToString(), DefaultPath, OutputDirectory))
		{
			TArray<ULocalizationTarget*> TargetObjectsToProcess;
			for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
			{
				if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0)
				{
					TargetObjectsToProcess.Add(LocalizationTarget);
				}
			}

			LocalizationCommandletTasks::ImportTextForTargets(ParentWindow.ToSharedRef(), TargetObjectsToProcess, TOptional<FString>(OutputDirectory));

			for (ULocalizationTarget* const LocalizationTarget : TargetObjectsToProcess)
			{
				UpdateTargetFromReports(LocalizationTarget);
			}
		}
	}
}

bool FLocalizationTargetSetDetailCustomization::CanExportTextAllTargets() const
{
	for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
	{
		// If any target can be exported, then exporting all can be executed.
		if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
		{
			return true;
		}
	}

	return false;
}

void FLocalizationTargetSetDetailCustomization::ExportTextAllTargets()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString DefaultPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir() / TEXT("Localization"));

		// Prompt the user for the directory
		FString OutputDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, LOCTEXT("ExportAllTranslationsDialogTitle", "Export All Translations to Directory").ToString(), DefaultPath, OutputDirectory))
		{
			TArray<ULocalizationTarget*> TargetObjectsToProcess;
			for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
			{
				if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0)
				{
					TargetObjectsToProcess.Add(LocalizationTarget);
				}
			}

			LocalizationCommandletTasks::ExportTextForTargets(ParentWindow.ToSharedRef(), TargetObjectsToProcess, TOptional<FString>(OutputDirectory));
		}
	}
}

bool FLocalizationTargetSetDetailCustomization::CanImportDialogueScriptAllTargets() const
{
	for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
	{
		// If any target can be imported, then importing dialogue can be executed.
		if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
		{
			return true;
		}
	}

	return false;
}

void FLocalizationTargetSetDetailCustomization::ImportDialogueScriptAllTargets()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}


		const FString DefaultPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir() / TEXT("Localization"));

		// Prompt the user for the directory
		FString OutputDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, LOCTEXT("ImportAllDialogueScriptsDialogTitle", "Import All Dialogue Scripts from Directory").ToString(), DefaultPath, OutputDirectory))
		{
			TArray<ULocalizationTarget*> TargetObjectsToProcess;
			for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
			{
				if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0)
				{
					TargetObjectsToProcess.Add(LocalizationTarget);
				}
			}

			LocalizationCommandletTasks::ImportDialogueScriptForTargets(ParentWindow.ToSharedRef(), TargetObjectsToProcess, TOptional<FString>(OutputDirectory));

			for (ULocalizationTarget* const LocalizationTarget : TargetObjectsToProcess)
			{
				UpdateTargetFromReports(LocalizationTarget);
			}
		}
	}
}

bool FLocalizationTargetSetDetailCustomization::CanExportDialogueScriptAllTargets() const
{
	for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
	{
		// If any target can be exported, then exporting dialogue can be executed.
		if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
		{
			return true;
		}
	}

	return false;
}

void FLocalizationTargetSetDetailCustomization::ExportDialogueScriptAllTargets()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString DefaultPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir() / TEXT("Localization"));

		// Prompt the user for the directory
		FString OutputDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, LOCTEXT("ExportAllDialogueScriptsDialogTitle", "Export All Dialogue Scripts to Directory").ToString(), DefaultPath, OutputDirectory))
		{
			TArray<ULocalizationTarget*> TargetObjectsToProcess;
			for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
			{
				if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0)
				{
					TargetObjectsToProcess.Add(LocalizationTarget);
				}
			}

			LocalizationCommandletTasks::ExportDialogueScriptForTargets(ParentWindow.ToSharedRef(), TargetObjectsToProcess, TOptional<FString>(OutputDirectory));
		}
	}
}

bool FLocalizationTargetSetDetailCustomization::CanImportDialogueAllTargets() const
{
	for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
	{
		// If any target can be imported, then importing dialogue can be executed.
		if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
		{
			return true;
		}
	}

	return false;
}

void FLocalizationTargetSetDetailCustomization::ImportDialogueAllTargets()
{
	TArray<ULocalizationTarget*> TargetObjectsToProcess;
	for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
	{
		if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0)
		{
			TargetObjectsToProcess.Add(LocalizationTarget);
		}
	}

	// Warn about potentially loaded audio assets
	if (!LocalizationCommandletTasks::ReportLoadedAudioAssets(TargetObjectsToProcess))
	{
		return;
	}

	// Execute import dialogue.
	const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
	LocalizationCommandletTasks::ImportDialogueForTargets(ParentWindow.ToSharedRef(), TargetObjectsToProcess);
}

bool FLocalizationTargetSetDetailCustomization::CanCountWordsForAllTargets() const
{
	for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
	{
		// If any target can be imported, then counting words for all can be executed.
		if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
		{
			return true;
		}
	}

	return false;
}

void FLocalizationTargetSetDetailCustomization::CountWordsForAllTargets()
{
	TArray<ULocalizationTarget*> TargetObjectsToProcess;
	for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
	{
		if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0)
		{
			TargetObjectsToProcess.Add(LocalizationTarget);
		}
	}

	// Execute compile.
	const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
	LocalizationCommandletTasks::GenerateWordCountReportsForTargets(ParentWindow.ToSharedRef(), TargetObjectsToProcess);
}

bool FLocalizationTargetSetDetailCustomization::CanCompileTextAllTargets() const
{
	for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
	{
		// If any target can be imported, then compiling all can be executed.
		if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
		{
			return true;
		}
	}

	return false;
}

void FLocalizationTargetSetDetailCustomization::CompileTextAllTargets()
{
	TArray<ULocalizationTarget*> TargetObjectsToProcess;
	for (ULocalizationTarget* const LocalizationTarget : TargetSet->TargetObjects)
	{
		if (LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0)
		{
			TargetObjectsToProcess.Add(LocalizationTarget);
		}
	}

	// Execute compile.
	const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(DetailLayoutBuilder->GetDetailsViewSharedPtr().ToSharedRef());
	LocalizationCommandletTasks::CompileTextForTargets(ParentWindow.ToSharedRef(), TargetObjectsToProcess);
}

void FLocalizationTargetSetDetailCustomization::UpdateTargetFromReports(ULocalizationTarget* const LocalizationTarget)
{
	//TArray< TSharedPtr<IPropertyHandle> > WordCountPropertyHandles;

	//const TSharedPtr<IPropertyHandle> TargetSettingsPropertyHandle = TargetEditor->GetTargetSettingsPropertyHandle();
	//if (TargetSettingsPropertyHandle.IsValid() && TargetSettingsPropertyHandle->IsValidHandle())
	//{
	//	const TSharedPtr<IPropertyHandle> NativeCultureStatisticsPropertyHandle = TargetSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, NativeCultureStatistics));
	//	if (NativeCultureStatisticsPropertyHandle.IsValid() && NativeCultureStatisticsPropertyHandle->IsValidHandle())
	//	{
	//		const TSharedPtr<IPropertyHandle> WordCountPropertyHandle = NativeCultureStatisticsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCultureStatistics, WordCount));
	//		if (WordCountPropertyHandle.IsValid() && WordCountPropertyHandle->IsValidHandle())
	//		{
	//			WordCountPropertyHandles.Add(WordCountPropertyHandle);
	//		}
	//	}

	//	const TSharedPtr<IPropertyHandle> SupportedCulturesStatisticsPropertyHandle = TargetSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, SupportedCulturesStatistics));
	//	if (SupportedCulturesStatisticsPropertyHandle.IsValid() && SupportedCulturesStatisticsPropertyHandle->IsValidHandle())
	//	{
	//		uint32 SupportedCultureCount = 0;
	//		SupportedCulturesStatisticsPropertyHandle->GetNumChildren(SupportedCultureCount);
	//		for (uint32 i = 0; i < SupportedCultureCount; ++i)
	//		{
	//			const TSharedPtr<IPropertyHandle> ElementPropertyHandle = SupportedCulturesStatisticsPropertyHandle->GetChildHandle(i);
	//			if (ElementPropertyHandle.IsValid() && ElementPropertyHandle->IsValidHandle())
	//			{
	//				const TSharedPtr<IPropertyHandle> WordCountPropertyHandle = SupportedCulturesStatisticsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCultureStatistics, WordCount));
	//				if (WordCountPropertyHandle.IsValid() && WordCountPropertyHandle->IsValidHandle())
	//				{
	//					WordCountPropertyHandles.Add(WordCountPropertyHandle);
	//				}
	//			}
	//		}
	//	}
	//}

	//for (const TSharedPtr<IPropertyHandle>& WordCountPropertyHandle : WordCountPropertyHandles)
	//{
	//	WordCountPropertyHandle->NotifyPreChange();
	//}
	LocalizationTarget->UpdateWordCountsFromCSV();
	LocalizationTarget->UpdateStatusFromConflictReport();
	//for (const TSharedPtr<IPropertyHandle>& WordCountPropertyHandle : WordCountPropertyHandles)
	//{
	//	WordCountPropertyHandle->NotifyPostChange();
	//}
}

TSharedRef<ITableRow> FLocalizationTargetSetDetailCustomization::OnGenerateRow(TSharedPtr<IPropertyHandle> TargetObjectPropertyHandle, const TSharedRef<STableViewBase>& Table)
{
	return SNew(SLocalizationDashboardTargetRow, Table, DetailLayoutBuilder->GetPropertyUtilities(), TargetObjectPropertyHandle.ToSharedRef());
}

FReply FLocalizationTargetSetDetailCustomization::OnNewTargetButtonClicked()
{
	if(TargetObjectsPropertyHandle.IsValid() && TargetObjectsPropertyHandle->IsValidHandle())
	{
		uint32 NewElementIndex;
		TargetObjectsPropertyHandle->AsArray()->GetNumElements(NewElementIndex);
		NewEntryIndexToBeInitialized = NewElementIndex;
		TargetObjectsPropertyHandle->AsArray()->AddItem();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
