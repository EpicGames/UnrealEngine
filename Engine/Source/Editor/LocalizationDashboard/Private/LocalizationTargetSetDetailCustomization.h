// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Views/SListView.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class ITableRow;
class STableViewBase;
class ULocalizationTarget;
class ULocalizationTargetSet;

class FLocalizationTargetSetDetailCustomization : public IDetailCustomization
{
public:
	FLocalizationTargetSetDetailCustomization();
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	void BuildTargetsList();
	void RebuildTargetsList();

	bool CanGatherTextAllTargets() const;
	void GatherTextAllTargets();

	bool CanImportTextAllTargets() const;
	void ImportTextAllTargets();

	bool CanExportTextAllTargets() const;
	void ExportTextAllTargets();

	bool CanImportDialogueScriptAllTargets() const;
	void ImportDialogueScriptAllTargets();

	bool CanExportDialogueScriptAllTargets() const;
	void ExportDialogueScriptAllTargets();

	bool CanImportDialogueAllTargets() const;
	void ImportDialogueAllTargets();

	bool CanCountWordsForAllTargets() const;
	void CountWordsForAllTargets();

	bool CanCompileTextAllTargets() const;
	void CompileTextAllTargets();

	void UpdateTargetFromReports(ULocalizationTarget* const LocalizationTarget);

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<IPropertyHandle> TargetObjectPropertyHandle, const TSharedRef<STableViewBase>& Table);
	FReply OnNewTargetButtonClicked();

private:
	IDetailLayoutBuilder* DetailLayoutBuilder;

	TWeakObjectPtr<ULocalizationTargetSet> TargetSet;

	TSharedPtr<IPropertyHandle> TargetObjectsPropertyHandle;
	FSimpleDelegate TargetsArrayPropertyHandle_OnNumElementsChanged;
	TArray< TSharedPtr<IPropertyHandle> > TargetsList;
	TSharedPtr< SListView< TSharedPtr<IPropertyHandle> > > TargetsListView;

	/* If set, the entry at the index specified needs to be initialized as soon as possible. */
	int32 NewEntryIndexToBeInitialized;
};
