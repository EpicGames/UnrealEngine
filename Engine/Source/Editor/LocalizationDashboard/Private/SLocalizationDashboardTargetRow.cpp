// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLocalizationDashboardTargetRow.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DesktopPlatformModule.h"
#include "FileHelpers.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericWindow.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformMisc.h"
#include "IDesktopPlatform.h"
#include "IPropertyUtilities.h"
#include "Internationalization/Culture.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/Internationalization.h"
#include "LocalizationCommandletTasks.h"
#include "LocalizationConfigurationScript.h"
#include "LocalizationDashboard.h"
#include "LocalizationTargetTypes.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SLocalizationTargetStatusButton.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Object.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

class STableViewBase;

#define LOCTEXT_NAMESPACE "LocalizationDashboardTargetRow"

void SLocalizationDashboardTargetRow::Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<IPropertyUtilities>& InPropertyUtilities, const TSharedRef<IPropertyHandle>& InTargetObjectPropertyHandle)
{
	PropertyUtilities = InPropertyUtilities;
	TargetObjectPropertyHandle = InTargetObjectPropertyHandle;

	FSuperRowType::Construct(InArgs, OwnerTableView);
}

TSharedRef<SWidget> SLocalizationDashboardTargetRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	TSharedPtr<SWidget> Return;

	if (ColumnName == "Target")
	{
		// Target Name
		Return = SNew(SHyperlink)
			.Text(this, &SLocalizationDashboardTargetRow::GetTargetName)
			.OnNavigate(this, &SLocalizationDashboardTargetRow::OnNavigate);
	}
	else if(ColumnName == "Status")
	{
		ULocalizationTarget* const LocalizationTarget = GetTarget();
		if (LocalizationTarget)
		{
			// Status icon button.
			Return = SNew(SLocalizationTargetStatusButton, *LocalizationTarget);
		}
	}
	else if(ColumnName == "WordCount")
	{
		// Progress Bar and Word Counts
		Return = SNew(STextBlock)
			.Text(this, &SLocalizationDashboardTargetRow::GetWordCountText);
	}
	else if(ColumnName == "Actions")
	{
		TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);
		Return = HorizontalBox;

		// Delete Target
		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), TEXT("HoverHintOnly") )
				.ToolTipText(LOCTEXT("DeleteButtonLabel", "Delete this target."))
				.OnClicked(this, &SLocalizationDashboardTargetRow::EnqueueDeletion)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("LocalizationDashboard.DeleteTarget"))
				]
			];
	}

	return Return.IsValid() ? Return.ToSharedRef() : SNullWidget::NullWidget;
}

ULocalizationTarget* SLocalizationDashboardTargetRow::GetTarget() const
{
	if (TargetObjectPropertyHandle.IsValid() && TargetObjectPropertyHandle->IsValidHandle())
	{
		UObject* Object = nullptr;
		if(FPropertyAccess::Result::Success == TargetObjectPropertyHandle->GetValue(Object))
		{
			return Cast<ULocalizationTarget>(Object);
		}
	}
	return nullptr;
}

FLocalizationTargetSettings* SLocalizationDashboardTargetRow::GetTargetSettings() const
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return LocalizationTarget ? &(LocalizationTarget->Settings) : nullptr;
}

FText SLocalizationDashboardTargetRow::GetTargetName() const
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return LocalizationTarget ? FText::FromString(LocalizationTarget->Settings.Name) : FText::GetEmpty();
}

void SLocalizationDashboardTargetRow::OnNavigate()
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	if (LocalizationTarget)
	{
		FLocalizationDashboard* const LocalizationDashboard = FLocalizationDashboard::Get();

		if (LocalizationDashboard)
		{
			TargetEditorDockTab = LocalizationDashboard->ShowTargetEditorTab(LocalizationTarget);
		}
	}
}

FText SLocalizationDashboardTargetRow::GetCulturesText() const
{
	FLocalizationTargetSettings* const TargetSettings = GetTargetSettings();
	if (TargetSettings)
	{
		TArray<FString> OrderedCultureNames;
		for (const auto& CultureStatistics : TargetSettings->SupportedCulturesStatistics)
		{
			OrderedCultureNames.Add(CultureStatistics.CultureName);
		}

		FString Result;
		for (FString& CultureName : OrderedCultureNames)
		{
			const FCulturePtr Culture = FInternationalization::Get().GetCulture(CultureName);
			if (Culture.IsValid())
			{
				const FString CultureDisplayName = Culture->GetDisplayName();

				if (!Result.IsEmpty())
				{
					Result.Append(TEXT(", "));
				}

				Result.Append(CultureDisplayName);
			}
		}

		return FText::FromString(Result);
	}

	return FText::GetEmpty();
}

uint32 SLocalizationDashboardTargetRow::GetWordCount() const
{
	FLocalizationTargetSettings* const TargetSettings = GetTargetSettings();
	return TargetSettings && TargetSettings->SupportedCulturesStatistics.IsValidIndex(TargetSettings->NativeCultureIndex) ? TargetSettings->SupportedCulturesStatistics[TargetSettings->NativeCultureIndex].WordCount : 0;
}

uint32 SLocalizationDashboardTargetRow::GetNativeWordCount() const
{
	FLocalizationTargetSettings* const TargetSettings = GetTargetSettings();
	return TargetSettings && TargetSettings->SupportedCulturesStatistics.IsValidIndex(TargetSettings->NativeCultureIndex) ? TargetSettings->SupportedCulturesStatistics[TargetSettings->NativeCultureIndex].WordCount : 0;
}

FText SLocalizationDashboardTargetRow::GetWordCountText() const
{
	return FText::AsNumber(GetWordCount());
}

void SLocalizationDashboardTargetRow::UpdateTargetFromReports()
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	if (LocalizationTarget)
	{
		//TArray< TSharedPtr<IPropertyHandle> > WordCountPropertyHandles;

		//const TSharedPtr<IPropertyHandle> TargetSettingsPropertyHandle = TargetEditor->GetTargetSettingsPropertyHandle();
		//if (TargetSettingsPropertyHandle.IsValid() && TargetSettingsPropertyHandle->IsValidHandle())
		//{
		//	const TSharedPtr<IPropertyHandle> NativeCultureStatisticsPropertyHandle = TargetSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, NativeCultureStatistics));
		//	if (NativeCultureStatisticsPropertyHandle.IsValid() && NativeCultureStatisticsPropertyHandle->IsValidHandle())
		//	{
		//		const TSharedPtr<IPropertyHandle> WordCountPropertyHandle = NativeCultureStatisticsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCultureStatistics, WordCount));
		//		if (WordCountPropertyHandle.IsValid() && WordCountPropertyHandle->IsValidHandle())
		//		{
		//			WordCountPropertyHandles.Add(WordCountPropertyHandle);
		//		}
		//	}

		//	const TSharedPtr<IPropertyHandle> SupportedCulturesStatisticsPropertyHandle = TargetSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, SupportedCulturesStatistics));
		//	if (SupportedCulturesStatisticsPropertyHandle.IsValid() && SupportedCulturesStatisticsPropertyHandle->IsValidHandle())
		//	{
		//		uint32 SupportedCultureCount = 0;
		//		SupportedCulturesStatisticsPropertyHandle->GetNumChildren(SupportedCultureCount);
		//		for (uint32 i = 0; i < SupportedCultureCount; ++i)
		//		{
		//			const TSharedPtr<IPropertyHandle> ElementPropertyHandle = SupportedCulturesStatisticsPropertyHandle->GetChildHandle(i);
		//			if (ElementPropertyHandle.IsValid() && ElementPropertyHandle->IsValidHandle())
		//			{
		//				const TSharedPtr<IPropertyHandle> WordCountPropertyHandle = SupportedCulturesStatisticsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCultureStatistics, WordCount));
		//				if (WordCountPropertyHandle.IsValid() && WordCountPropertyHandle->IsValidHandle())
		//				{
		//					WordCountPropertyHandles.Add(WordCountPropertyHandle);
		//				}
		//			}
		//		}
		//	}
		//}

		//for (const TSharedPtr<IPropertyHandle>& WordCountPropertyHandle : WordCountPropertyHandles)
		//{
		//	WordCountPropertyHandle->NotifyPreChange();
		//}
		LocalizationTarget->UpdateWordCountsFromCSV();
		LocalizationTarget->UpdateStatusFromConflictReport();
		//for (const TSharedPtr<IPropertyHandle>& WordCountPropertyHandle : WordCountPropertyHandles)
		//{
		//	WordCountPropertyHandle->NotifyPostChange();
		//}
	}
}

bool SLocalizationDashboardTargetRow::CanGatherText() const
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationDashboardTargetRow::GatherText()
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	if (LocalizationTarget)
	{
		// Save unsaved packages.
		const bool bPromptUserToSave = true;
		const bool bSaveMapPackages = true;
		const bool bSaveContentPackages = true;
		const bool bFastSave = false;
		const bool bNotifyNoPackagesSaved = false;
		const bool bCanBeDeclined = true;
		bool DidPackagesNeedSaving;
		const bool WerePackagesSaved = FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined, &DidPackagesNeedSaving);

		if (DidPackagesNeedSaving && !WerePackagesSaved)
		{
			// Give warning dialog.
			const FText MessageText = NSLOCTEXT("LocalizationCultureActions", "UnsavedPackagesWarningDialogMessage", "There are unsaved changes. These changes may not be gathered from correctly.");
			const FText TitleText = NSLOCTEXT("LocalizationCultureActions", "UnsavedPackagesWarningDialogTitle", "Unsaved Changes Before Gather");
			switch(FMessageDialog::Open(EAppMsgType::OkCancel, MessageText, TitleText))
			{
			case EAppReturnType::Cancel:
				{
					return FReply::Handled();
				}
			}
		}

		// Execute gather.
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		LocalizationCommandletTasks::GatherTextForTarget(ParentWindow.ToSharedRef(), LocalizationTarget);

		UpdateTargetFromReports();
	}

	return FReply::Handled();
}

bool SLocalizationDashboardTargetRow::CanImportText() const
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationDashboardTargetRow::ImportText()
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (LocalizationTarget && DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString DefaultPath = FPaths::ConvertRelativePathToFull(LocalizationConfigurationScript::GetDataDirectory(LocalizationTarget));

		FText DialogTitle;
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			DialogTitle = FText::Format(LOCTEXT("ImportAllTranslationsForTargetDialogTitleFormat", "Import All Translations for {TargetName} from Directory"), FormatArguments);
		}

		// Prompt the user for the directory
		FString OutputDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, DialogTitle.ToString(), DefaultPath, OutputDirectory))
		{
			LocalizationCommandletTasks::ImportTextForTarget(ParentWindow.ToSharedRef(), LocalizationTarget, TOptional<FString>(OutputDirectory));
			UpdateTargetFromReports();
		}
	}

	return FReply::Handled();
}

bool SLocalizationDashboardTargetRow::CanExportText() const
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationDashboardTargetRow::ExportText()
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (LocalizationTarget && DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString DefaultPath = FPaths::ConvertRelativePathToFull(LocalizationConfigurationScript::GetDataDirectory(LocalizationTarget));

		FText DialogTitle;
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			DialogTitle = FText::Format(LOCTEXT("ExportAllTranslationsForTargetDialogTitleFormat", "Export All Translations for {TargetName} to Directory"), FormatArguments);
		}

		// Prompt the user for the directory
		FString OutputDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, DialogTitle.ToString(), DefaultPath, OutputDirectory))
		{
			LocalizationCommandletTasks::ExportTextForTarget(ParentWindow.ToSharedRef(), LocalizationTarget, TOptional<FString>(OutputDirectory));
		}
	}

	return FReply::Handled();
}

bool SLocalizationDashboardTargetRow::CanImportDialogueScript() const
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationDashboardTargetRow::ImportDialogueScript()
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (LocalizationTarget && DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString DefaultPath = FPaths::ConvertRelativePathToFull(LocalizationConfigurationScript::GetDataDirectory(LocalizationTarget));

		FText DialogTitle;
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			DialogTitle = FText::Format(LOCTEXT("ImportAllDialogueScriptsForTargetDialogTitleFormat", "Import All Dialogue Scripts for {TargetName} from Directory"), FormatArguments);
		}

		// Prompt the user for the directory
		FString OutputDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, DialogTitle.ToString(), DefaultPath, OutputDirectory))
		{
			LocalizationCommandletTasks::ImportDialogueScriptForTarget(ParentWindow.ToSharedRef(), LocalizationTarget, TOptional<FString>(OutputDirectory));
			UpdateTargetFromReports();
		}
	}

	return FReply::Handled();
}

bool SLocalizationDashboardTargetRow::CanExportDialogueScript() const
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationDashboardTargetRow::ExportDialogueScript()
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (LocalizationTarget && DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString DefaultPath = FPaths::ConvertRelativePathToFull(LocalizationConfigurationScript::GetDataDirectory(LocalizationTarget));

		FText DialogTitle;
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			DialogTitle = FText::Format(LOCTEXT("ExportAllDialogueScriptsForTargetDialogTitleFormat", "Export All Dialogue Scripts for {TargetName} to Directory"), FormatArguments);
		}

		// Prompt the user for the directory
		FString OutputDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, DialogTitle.ToString(), DefaultPath, OutputDirectory))
		{
			LocalizationCommandletTasks::ExportDialogueScriptForTarget(ParentWindow.ToSharedRef(), LocalizationTarget, TOptional<FString>(OutputDirectory));
		}
	}

	return FReply::Handled();
}

bool SLocalizationDashboardTargetRow::CanImportDialogue() const
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationDashboardTargetRow::ImportDialogue()
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	if (LocalizationTarget)
	{
		// Warn about potentially loaded audio assets
		{
			TArray<ULocalizationTarget*> Targets;
			Targets.Add(LocalizationTarget);

			if (!LocalizationCommandletTasks::ReportLoadedAudioAssets(Targets))
			{
				return FReply::Handled();
			}
		}

		// Execute import dialogue.
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		LocalizationCommandletTasks::ImportDialogueForTarget(ParentWindow.ToSharedRef(), LocalizationTarget);
	}

	return FReply::Handled();
}

bool SLocalizationDashboardTargetRow::CanCountWords() const
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationDashboardTargetRow::CountWords()
{
	bool HasSucceeded = false;

	ULocalizationTarget* const LocalizationTarget = GetTarget();
	if (LocalizationTarget)
	{
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		LocalizationCommandletTasks::GenerateWordCountReportForTarget(ParentWindow.ToSharedRef(), LocalizationTarget);

		UpdateTargetFromReports();
	}

	return FReply::Handled();
}

bool SLocalizationDashboardTargetRow::CanCompileText() const
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationDashboardTargetRow::CompileText()
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	if (LocalizationTarget)
	{
		// Execute compile.
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		LocalizationCommandletTasks::CompileTextForTarget(ParentWindow.ToSharedRef(), LocalizationTarget);
	}

	return FReply::Handled();
}

FReply SLocalizationDashboardTargetRow::EnqueueDeletion()
{
	PropertyUtilities->EnqueueDeferredAction(FSimpleDelegate::CreateSP(this, &SLocalizationDashboardTargetRow::Delete));
	return FReply::Handled();
}

void SLocalizationDashboardTargetRow::Delete()
{
	static bool IsExecuting = false;
	if (!IsExecuting)
	{
		TGuardValue<bool> ReentranceGuard(IsExecuting, true);

		ULocalizationTarget* const LocalizationTarget = GetTarget();
		if (LocalizationTarget)
		{
			// Setup dialog for deleting target.
			const FText FormatPattern = NSLOCTEXT("LocalizationDashboard", "DeleteTargetConfirmationDialogMessage", "This action can not be undone and data will be permanently lost. Are you sure you would like to delete {TargetName}?");
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			const FText MessageText = FText::Format(FormatPattern, Arguments);
			const FText TitleText = NSLOCTEXT("LocalizationDashboard", "DeleteTargetConfirmationDialogTitle", "Confirm Target Deletion");

			switch(FMessageDialog::Open(EAppMsgType::OkCancel, MessageText, TitleText))
			{
			case EAppReturnType::Ok:
				{
					LocalizationTarget->DeleteFiles();

					// Close target editor.
					if (TargetEditorDockTab.IsValid())
					{
						const TSharedPtr<SDockTab> OldTargetEditorTab = TargetEditorDockTab.Pin();
						OldTargetEditorTab->RequestCloseTab();
					}

					// Remove this element from the parent array.
					const TSharedPtr<IPropertyHandle> ParentPropertyHandle = TargetObjectPropertyHandle->GetParentHandle();
					if (ParentPropertyHandle.IsValid() && ParentPropertyHandle->IsValidHandle())
					{
						const TSharedPtr<IPropertyHandleArray> ParentArrayPropertyHandle = ParentPropertyHandle->AsArray();
						if (ParentArrayPropertyHandle.IsValid())
						{
							ParentArrayPropertyHandle->DeleteItem(TargetObjectPropertyHandle->GetIndexInArray());
						}
					}
				}
				break;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
