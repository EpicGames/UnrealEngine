// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/BitArray.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class IPropertyHandle;
class IPropertyUtilities;
class SDockTab;
class STableViewBase;
class SWidget;
class ULocalizationTarget;
struct FLocalizationTargetSettings;

class SLocalizationDashboardTargetRow : public SMultiColumnTableRow< TSharedPtr<IPropertyHandle> >
{
public:
	void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<IPropertyUtilities>& InPropertyUtilities, const TSharedRef<IPropertyHandle>& InTargetObjectPropertyHandle);
	TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;

private:
	ULocalizationTarget* GetTarget() const;
	FLocalizationTargetSettings* GetTargetSettings() const;

	FText GetTargetName() const;
	void OnNavigate();

	FText GetCulturesText() const;

	uint32 GetWordCount() const;
	uint32 GetNativeWordCount() const;
	FText GetWordCountText() const;

	void UpdateTargetFromReports();

	bool CanGatherText() const;
	FReply GatherText();

	bool CanImportText() const;
	FReply ImportText();

	bool CanExportText() const;
	FReply ExportText();

	bool CanImportDialogueScript() const;
	FReply ImportDialogueScript();

	bool CanExportDialogueScript() const;
	FReply ExportDialogueScript();

	bool CanImportDialogue() const;
	FReply ImportDialogue();

	bool CanCountWords() const;
	FReply CountWords();

	bool CanCompileText() const;
	FReply CompileText();

	FReply EnqueueDeletion();
	void Delete();

private:
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
	TSharedPtr<IPropertyHandle> TargetObjectPropertyHandle;
	TWeakPtr<SDockTab> TargetEditorDockTab;
};
