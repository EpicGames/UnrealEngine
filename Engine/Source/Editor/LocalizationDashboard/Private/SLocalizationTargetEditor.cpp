// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLocalizationTargetEditor.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "Layout/Children.h"
#include "LocalizationConfigurationScript.h"
#include "LocalizationTargetTypes.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"

struct FPropertyChangedEvent;

void SLocalizationTargetEditor::Construct(const FArguments& InArgs, ULocalizationTargetSet* const InProjectSettings, ULocalizationTarget* const InLocalizationTarget, const FIsPropertyEditingEnabled& IsPropertyEditingEnabled)
{
	check(InProjectSettings->TargetObjects.Contains(InLocalizationTarget));
	LocalizationTarget = InLocalizationTarget;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	TSharedRef<IDetailsView> DetailsView = PropertyModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(InLocalizationTarget, true);
	DetailsView->SetIsPropertyEditingEnabledDelegate(IsPropertyEditingEnabled);
	DetailsView->OnFinishedChangingProperties().AddSP(this, &SLocalizationTargetEditor::OnFinishedChangingProperties);

	ChildSlot
	[
		DetailsView
	];
}

void SLocalizationTargetEditor::OnFinishedChangingProperties(const FPropertyChangedEvent& InEvent)
{
	if (ULocalizationTarget* LocalizationTargetPtr = LocalizationTarget.Get())
	{
		// Update the exported gather INIs for this target to reflect the new settings
		LocalizationConfigurationScript::GenerateAllConfigFiles(LocalizationTargetPtr);
	}
}
