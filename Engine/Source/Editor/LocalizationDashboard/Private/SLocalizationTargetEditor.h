// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyEditorDelegates.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ULocalizationTarget;
class ULocalizationTargetSet;
struct FPropertyChangedEvent;

class SLocalizationTargetEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLocalizationTargetEditor) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ULocalizationTargetSet* const InProjectSettings, ULocalizationTarget* const InTarget, const FIsPropertyEditingEnabled& IsPropertyEditingEnabled);

private:
	void OnFinishedChangingProperties(const FPropertyChangedEvent& InEvent);

	/** Localization target being edited */
	TWeakObjectPtr<ULocalizationTarget> LocalizationTarget;
};
