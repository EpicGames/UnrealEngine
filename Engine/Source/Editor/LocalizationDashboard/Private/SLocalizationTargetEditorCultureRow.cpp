// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLocalizationTargetEditorCultureRow.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Internationalization/Culture.h"
#include "Modules/ModuleManager.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "IPropertyUtilities.h"
#include "LocalizationTargetTypes.h"
#include "ITranslationEditor.h"
#include "LocalizationConfigurationScript.h"
#include "LocalizationCommandletTasks.h"

#define LOCTEXT_NAMESPACE "LocalizationTargetEditorCultureRow"

void SLocalizationTargetEditorCultureRow::Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<IPropertyUtilities>& InPropertyUtilities, const TSharedRef<IPropertyHandle>& InTargetSettingsPropertyHandle, const int32 InCultureIndex)
{
	PropertyUtilities = InPropertyUtilities;
	TargetSettingsPropertyHandle = InTargetSettingsPropertyHandle;
	CultureIndex = InCultureIndex;

	FSuperRowType::Construct(InArgs, OwnerTableView);
}

TSharedRef<SWidget> SLocalizationTargetEditorCultureRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	TSharedPtr<SWidget> Return;

	if (ColumnName == "IsNative")
	{
		Return = SNew(SCheckBox)
			.Style(&FCoreStyle::Get().GetWidgetStyle< FCheckBoxStyle >("RadioButton"))
			.IsChecked_Lambda([this](){return IsNativeCultureForTarget() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;})
			.OnCheckStateChanged(this, &SLocalizationTargetEditorCultureRow::OnNativeCultureCheckStateChanged);
	}
	else if (ColumnName == "Culture")
	{
		// Culture Name
		Return = SNew(STextBlock)
			.Text(this, &SLocalizationTargetEditorCultureRow::GetCultureDisplayName)
			.ToolTipText(this, &SLocalizationTargetEditorCultureRow::GetCultureName);
	}
	else if(ColumnName == "WordCount")
	{
		// Progress Bar and Word Counts
		Return = SNew(SOverlay)
			+SOverlay::Slot()
			.VAlign(VAlign_Fill)
			.Padding(0.0f)
			[
				SNew(SProgressBar)
				.Percent(this, &SLocalizationTargetEditorCultureRow::GetProgressPercentage)
			]
		+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(this, &SLocalizationTargetEditorCultureRow::GetWordCountText)
			];
	}
	else if(ColumnName == "Actions")
	{
		TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);
		Return = HorizontalBox;

		// Edit Text
		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), TEXT("HoverHintOnly") )
				.ToolTipText( NSLOCTEXT("LocalizationTargetCultureActions", "EditButtonLabel", "Edit translations for this culture.") )
				.IsEnabled(this, &SLocalizationTargetEditorCultureRow::CanEditText)
				.OnClicked(this, &SLocalizationTargetEditorCultureRow::EditText)
				.Content()
				[
					SNew(SImage)
					.Image( FAppStyle::GetBrush("LocalizationTargetEditor.EditTranslations") )
				]
			];

		// Import Text
		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), TEXT("HoverHintOnly") )
				.ToolTipText( NSLOCTEXT("LocalizationTargetCultureActions", "ImportTextButtonLabel", "Import translations for this culture.") )
				.IsEnabled(this, &SLocalizationTargetEditorCultureRow::CanImportText)
				.OnClicked(this, &SLocalizationTargetEditorCultureRow::ImportText)
				.Content()
				[
					SNew(SImage)
					.Image( FAppStyle::GetBrush("LocalizationTargetEditor.ImportTextCulture") )
				]
			];

		// Export Text
		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), TEXT("HoverHintOnly") )
				.ToolTipText(NSLOCTEXT("LocalizationTargetCultureActions", "ExportTextButtonLabel", "Export translations for this culture."))
				.IsEnabled(this, &SLocalizationTargetEditorCultureRow::CanExportText)
				.OnClicked(this, &SLocalizationTargetEditorCultureRow::ExportText)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("LocalizationTargetEditor.ExportTextCulture"))
				]
			];

		// Import Dialogue Script
		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), TEXT("HoverHintOnly"))
				.ToolTipText(NSLOCTEXT("LocalizationTargetCultureActions", "ImportDialogueScriptButtonLabel", "Import dialogue scripts for this culture."))
				.IsEnabled(this, &SLocalizationTargetEditorCultureRow::CanImportDialogueScript)
				.OnClicked(this, &SLocalizationTargetEditorCultureRow::ImportDialogueScript)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("LocalizationTargetEditor.ImportDialogueScriptCulture"))
				]
			];

		// Export Dialogue Script
		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), TEXT("HoverHintOnly"))
				.ToolTipText(NSLOCTEXT("LocalizationTargetCultureActions", "ExportDialogueScriptButtonLabel", "Export dialogue scripts for this culture."))
				.IsEnabled(this, &SLocalizationTargetEditorCultureRow::CanExportDialogueScript)
				.OnClicked(this, &SLocalizationTargetEditorCultureRow::ExportDialogueScript)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("LocalizationTargetEditor.ExportDialogueScriptCulture"))
				]
			];

		// Import Dialogue
		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), TEXT("HoverHintOnly"))
				.ToolTipText(NSLOCTEXT("LocalizationTargetCultureActions", "ImportDialogueButtonLabel", "Import dialogue WAV files for this culture."))
				.IsEnabled(this, &SLocalizationTargetEditorCultureRow::CanImportDialogue)
				.OnClicked(this, &SLocalizationTargetEditorCultureRow::ImportDialogue)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("LocalizationTargetEditor.ImportDialogueCulture"))
				]
			];

		// Compile Text
		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), TEXT("HoverHintOnly"))
				.ToolTipText(NSLOCTEXT("LocalizationTargetCultureActions", "CompileTextButtonLabel", "Compile translations for this culture."))
				.IsEnabled(this, &SLocalizationTargetEditorCultureRow::CanCompileText)
				.OnClicked(this, &SLocalizationTargetEditorCultureRow::CompileText)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("LocalizationTargetEditor.CompileTextCulture"))
				]
			];

		// Delete Culture
		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), TEXT("HoverHintOnly") )
				.ToolTipText(NSLOCTEXT("LocalizationTargetActions", "DeleteButtonLabel", "Delete this culture."))
				.IsEnabled(this, &SLocalizationTargetEditorCultureRow::CanDelete)
				.OnClicked(this, &SLocalizationTargetEditorCultureRow::EnqueueDeletion)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("LocalizationTargetEditor.DeleteCulture"))
				]
			];
	}

	return Return.IsValid() ? Return.ToSharedRef() : SNullWidget::NullWidget;
}

ULocalizationTarget* SLocalizationTargetEditorCultureRow::GetTarget() const
{
	if (TargetSettingsPropertyHandle.IsValid() && TargetSettingsPropertyHandle->IsValidHandle())
	{
		TArray<UObject*> OuterObjects;
		TargetSettingsPropertyHandle->GetOuterObjects(OuterObjects);
		return CastChecked<ULocalizationTarget>(OuterObjects.Top());
	}
	return nullptr;
}

FLocalizationTargetSettings* SLocalizationTargetEditorCultureRow::GetTargetSettings() const
{
	if (TargetSettingsPropertyHandle.IsValid() && TargetSettingsPropertyHandle->IsValidHandle())
	{
		TArray<void*> RawData;
		TargetSettingsPropertyHandle->AccessRawData(RawData);
		return RawData.Num() ? reinterpret_cast<FLocalizationTargetSettings*>(RawData.Top()) : nullptr;
	}
	return nullptr;
}

FCultureStatistics* SLocalizationTargetEditorCultureRow::GetCultureStatistics() const
{
	FLocalizationTargetSettings* const TargetSettings = GetTargetSettings();
	if (TargetSettings)
	{
		return &TargetSettings->SupportedCulturesStatistics[CultureIndex];
	}
	return nullptr;
}

FCulturePtr SLocalizationTargetEditorCultureRow::GetCulture() const
{
	FCultureStatistics* const CultureStatistics = GetCultureStatistics();
	if (CultureStatistics)
	{
		return FInternationalization::Get().GetCulture(CultureStatistics->CultureName);
	}
	return nullptr;
}

bool SLocalizationTargetEditorCultureRow::IsNativeCultureForTarget() const
{
	FLocalizationTargetSettings* const TargetSettings = GetTargetSettings();
	return TargetSettings->SupportedCulturesStatistics.IsValidIndex(TargetSettings->NativeCultureIndex) && &(TargetSettings->SupportedCulturesStatistics[TargetSettings->NativeCultureIndex]) == GetCultureStatistics();
}

void SLocalizationTargetEditorCultureRow::OnNativeCultureCheckStateChanged( const ECheckBoxState CheckState )
{
	FLocalizationTargetSettings* const TargetSettings = GetTargetSettings();
	
	if (TargetSettings && TargetSettingsPropertyHandle.IsValid() && TargetSettingsPropertyHandle->IsValidHandle() && CheckState == ECheckBoxState::Checked)
	{
		const FText FormatPattern = LOCTEXT("ChangingNativeCultureWarningMessage", "This action can not be undone and all translations be permanently lost. Are you sure you would like to set the native culture to {CultureName}?");
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("CultureName"), GetCultureDisplayName());
		const FText MessageText = FText::Format(FormatPattern, Arguments);
		const FText TitleText = LOCTEXT("ChangingNativeCultureWarningDialogTitle", "Change Native Culture");

		switch(FMessageDialog::Open(EAppMsgType::YesNo, MessageText, TitleText))
		{
		case EAppReturnType::Yes:
			{
				ULocalizationTarget* const LocalizationTarget = GetTarget();
				if (LocalizationTarget)
				{
					// Delete data files.
					const FString DataDirectory = LocalizationConfigurationScript::GetDataDirectory(LocalizationTarget);
					IFileManager::Get().DeleteDirectory(*DataDirectory, false, true);
				}

				UpdateTargetFromReports();

				TargetSettingsPropertyHandle->NotifyPreChange();
				TargetSettings->NativeCultureIndex = CultureIndex;
				TargetSettingsPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			}
			break;
		}
	}
}

uint32 SLocalizationTargetEditorCultureRow::GetWordCount() const
{
	FCultureStatistics* const CultureStatistics = GetCultureStatistics();
	if (CultureStatistics)
	{
		return CultureStatistics->WordCount;
	}
	return 0;
}

uint32 SLocalizationTargetEditorCultureRow::GetNativeWordCount() const
{
	FLocalizationTargetSettings* const TargetSettings = GetTargetSettings();
	return TargetSettings && TargetSettings->SupportedCulturesStatistics.IsValidIndex(TargetSettings->NativeCultureIndex) ? TargetSettings->SupportedCulturesStatistics[TargetSettings->NativeCultureIndex].WordCount : 0;
}

FText SLocalizationTargetEditorCultureRow::GetCultureDisplayName() const
{
	const FCulturePtr Culture = GetCulture();
	return Culture.IsValid() ? FText::FromString(Culture->GetDisplayName()) : FText::GetEmpty();
}

FText SLocalizationTargetEditorCultureRow::GetCultureName() const
{
	const FCulturePtr Culture = GetCulture();
	return Culture.IsValid() ? FText::FromString(Culture->GetName()) : FText::GetEmpty();
}

FText SLocalizationTargetEditorCultureRow::GetWordCountText() const
{
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("TranslatedWordCount"), FText::AsNumber(GetWordCount()));
	TOptional<float> Percentage = GetProgressPercentage();
	Arguments.Add(TEXT("TranslatedPercentage"), FText::AsPercent(Percentage.IsSet() ? Percentage.GetValue() : 0.0f));
	return FText::Format(LOCTEXT("CultureWordCountProgressFormat", "{TranslatedWordCount} ({TranslatedPercentage})"), Arguments);
}

TOptional<float> SLocalizationTargetEditorCultureRow::GetProgressPercentage() const
{
	uint32 WordCount = GetWordCount();
	uint32 NativeWordCount = GetNativeWordCount();
	return IsNativeCultureForTarget() ? 1.0f : NativeWordCount != 0 ? float(WordCount) / float(NativeWordCount) : 0.0f;
}

void SLocalizationTargetEditorCultureRow::UpdateTargetFromReports()
{
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	if (LocalizationTarget)
	{
		TArray< TSharedPtr<IPropertyHandle> > WordCountPropertyHandles;

		if (TargetSettingsPropertyHandle.IsValid() && TargetSettingsPropertyHandle->IsValidHandle())
		{
			const TSharedPtr<IPropertyHandle> SupportedCulturesStatisticsPropertyHandle = TargetSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, SupportedCulturesStatistics));
			if (SupportedCulturesStatisticsPropertyHandle.IsValid() && SupportedCulturesStatisticsPropertyHandle->IsValidHandle())
			{
				uint32 SupportedCultureCount = 0;
				SupportedCulturesStatisticsPropertyHandle->GetNumChildren(SupportedCultureCount);
				for (uint32 i = 0; i < SupportedCultureCount; ++i)
				{
					const TSharedPtr<IPropertyHandle> ElementPropertyHandle = SupportedCulturesStatisticsPropertyHandle->GetChildHandle(i);
					if (ElementPropertyHandle.IsValid() && ElementPropertyHandle->IsValidHandle())
					{
						const TSharedPtr<IPropertyHandle> WordCountPropertyHandle = SupportedCulturesStatisticsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCultureStatistics, WordCount));
						if (WordCountPropertyHandle.IsValid() && WordCountPropertyHandle->IsValidHandle())
						{
							WordCountPropertyHandles.Add(WordCountPropertyHandle);
						}
					}
				}
			}
		}

		for (const TSharedPtr<IPropertyHandle>& WordCountPropertyHandle : WordCountPropertyHandles)
		{
			WordCountPropertyHandle->NotifyPreChange();
		}
		LocalizationTarget->UpdateWordCountsFromCSV();
		LocalizationTarget->UpdateStatusFromConflictReport();
		for (const TSharedPtr<IPropertyHandle>& WordCountPropertyHandle : WordCountPropertyHandles)
		{
			WordCountPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
	}
}

bool SLocalizationTargetEditorCultureRow::CanEditText() const
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return Culture.IsValid() && LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationTargetEditorCultureRow::EditText()
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	if (Culture.IsValid() && LocalizationTarget)
	{
		FString NativeCultureName;
		if (LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex))
		{
			NativeCultureName = LocalizationTarget->Settings.SupportedCulturesStatistics[LocalizationTarget->Settings.NativeCultureIndex].CultureName;
		}
		
		FModuleManager::Get().LoadModuleChecked("TranslationEditor");
		ITranslationEditor::OpenTranslationEditor(LocalizationTarget, Culture->GetName());
	}

	return FReply::Handled();
}

bool SLocalizationTargetEditorCultureRow::CanImportText() const
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return Culture.IsValid() && LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationTargetEditorCultureRow::ImportText()
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (Culture.IsValid() && LocalizationTarget && DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		{
			const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
			if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
			{
				ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
			}
		}

		const FString POFileName = LocalizationConfigurationScript::GetDefaultPOFileName(LocalizationTarget);
		const FString POFileTypeDescription = LOCTEXT("PortableObjectFileDescription", "Portable Object").ToString();
		const FString POFileExtension = FPaths::GetExtension(POFileName);
		const FString POFileExtensionWildcard = FString::Printf(TEXT("*.%s"), *POFileExtension);
		const FString FileTypes = FString::Printf(TEXT("%s (%s)|%s"), *POFileTypeDescription, *POFileExtensionWildcard, *POFileExtensionWildcard);
		const FString DefaultFilename = POFileName;
		const FString DefaultPath = FPaths::GetPath(LocalizationConfigurationScript::GetDefaultPOPath(LocalizationTarget, Culture->GetName()));

		FText DialogTitle;
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			FormatArguments.Add(TEXT("CultureName"), FText::FromString(Culture->GetDisplayName()));
			DialogTitle = FText::Format(LOCTEXT("ImportSpecificTranslationsForTargetDialogTitleFormat", "Import {CultureName} Translations for {TargetName} from Directory"), FormatArguments);
		}

		// Prompt the user for the directory
		TArray<FString> OpenFilenames;
		if (DesktopPlatform->OpenFileDialog(ParentWindowWindowHandle, DialogTitle.ToString(), DefaultPath, DefaultFilename, FileTypes, 0, OpenFilenames))
		{
			const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
			LocalizationCommandletTasks::ImportTextForCulture(ParentWindow.ToSharedRef(), LocalizationTarget, Culture->GetName(), TOptional<FString>(OpenFilenames.Top()));

			UpdateTargetFromReports();
		}
	}

	return FReply::Handled();
}

bool SLocalizationTargetEditorCultureRow::CanExportText() const
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return Culture.IsValid() && LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationTargetEditorCultureRow::ExportText()
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (Culture.IsValid() && LocalizationTarget && DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString POFileName = LocalizationConfigurationScript::GetDefaultPOFileName(LocalizationTarget);
		const FString POFileTypeDescription = LOCTEXT("PortableObjectFileDescription", "Portable Object").ToString();
		const FString POFileExtension = FPaths::GetExtension(POFileName);
		const FString POFileExtensionWildcard = FString::Printf(TEXT("*.%s"), *POFileExtension);
		const FString FileTypes = FString::Printf(TEXT("%s (%s)|%s"), *POFileTypeDescription, *POFileExtensionWildcard, *POFileExtensionWildcard);
		const FString DefaultFilename = POFileName;
		const FString DefaultPath = FPaths::GetPath(LocalizationConfigurationScript::GetDefaultPOPath(LocalizationTarget, Culture->GetName()));

		FText DialogTitle;
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			FormatArguments.Add(TEXT("CultureName"), FText::FromString(Culture->GetDisplayName()));
			DialogTitle = FText::Format(LOCTEXT("ExportSpecificTranslationsForTargetDialogTitleFormat", "Export {CultureName} Translations for {TargetName} to Directory"), FormatArguments);
		}

		// Prompt the user for the directory
		TArray<FString> SaveFilenames;
		if (DesktopPlatform->SaveFileDialog(ParentWindowWindowHandle, DialogTitle.ToString(), DefaultPath, DefaultFilename, FileTypes, 0, SaveFilenames))
		{
			LocalizationCommandletTasks::ExportTextForCulture(ParentWindow.ToSharedRef(), LocalizationTarget, Culture->GetName(), TOptional<FString>(SaveFilenames.Top()));
		}
	}

	return FReply::Handled();
}

bool SLocalizationTargetEditorCultureRow::CanImportDialogueScript() const
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return Culture.IsValid() && LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationTargetEditorCultureRow::ImportDialogueScript()
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (Culture.IsValid() && LocalizationTarget && DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		{
			const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
			if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
			{
				ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
			}
		}

		const FString DialogueScriptFileName = LocalizationConfigurationScript::GetDefaultDialogueScriptFileName(LocalizationTarget);
		const FString DialogueScriptFileTypeDescription = LOCTEXT("DialogueScriptFileDescription", "Dialogue Script CSV").ToString();
		const FString DialogueScriptFileExtension = FPaths::GetExtension(DialogueScriptFileName);
		const FString DialogueScriptFileExtensionWildcard = FString::Printf(TEXT("*.%s"), *DialogueScriptFileExtension);
		const FString FileTypes = FString::Printf(TEXT("%s (%s)|%s"), *DialogueScriptFileTypeDescription, *DialogueScriptFileExtensionWildcard, *DialogueScriptFileExtensionWildcard);
		const FString DefaultFilename = DialogueScriptFileName;
		const FString DefaultPath = FPaths::GetPath(LocalizationConfigurationScript::GetDefaultDialogueScriptPath(LocalizationTarget, Culture->GetName()));

		FText DialogTitle;
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			FormatArguments.Add(TEXT("CultureName"), FText::FromString(Culture->GetDisplayName()));
			DialogTitle = FText::Format(LOCTEXT("ImportSpecificDialogueScriptsForTargetDialogTitleFormat", "Import {CultureName} Dialogue Scripts for {TargetName} from Directory"), FormatArguments);
		}

		// Prompt the user for the directory
		TArray<FString> OpenFilenames;
		if (DesktopPlatform->OpenFileDialog(ParentWindowWindowHandle, DialogTitle.ToString(), DefaultPath, DefaultFilename, FileTypes, 0, OpenFilenames))
		{
			const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
			LocalizationCommandletTasks::ImportDialogueScriptForCulture(ParentWindow.ToSharedRef(), LocalizationTarget, Culture->GetName(), TOptional<FString>(OpenFilenames.Top()));

			UpdateTargetFromReports();
		}
	}

	return FReply::Handled();
}

bool SLocalizationTargetEditorCultureRow::CanExportDialogueScript() const
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return Culture.IsValid() && LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationTargetEditorCultureRow::ExportDialogueScript()
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (Culture.IsValid() && LocalizationTarget && DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString DialogueScriptFileName = LocalizationConfigurationScript::GetDefaultDialogueScriptFileName(LocalizationTarget);
		const FString DialogueScriptFileTypeDescription = LOCTEXT("DialogueScriptFileDescription", "Dialogue Script CSV").ToString();
		const FString DialogueScriptFileExtension = FPaths::GetExtension(DialogueScriptFileName);
		const FString DialogueScriptFileExtensionWildcard = FString::Printf(TEXT("*.%s"), *DialogueScriptFileExtension);
		const FString FileTypes = FString::Printf(TEXT("%s (%s)|%s"), *DialogueScriptFileTypeDescription, *DialogueScriptFileExtensionWildcard, *DialogueScriptFileExtensionWildcard);
		const FString DefaultFilename = DialogueScriptFileName;
		const FString DefaultPath = FPaths::GetPath(LocalizationConfigurationScript::GetDefaultDialogueScriptPath(LocalizationTarget, Culture->GetName()));

		FText DialogTitle;
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("TargetName"), FText::FromString(LocalizationTarget->Settings.Name));
			FormatArguments.Add(TEXT("CultureName"), FText::FromString(Culture->GetDisplayName()));
			DialogTitle = FText::Format(LOCTEXT("ExportSpecificDialogueScriptsForTargetDialogTitleFormat", "Export {CultureName} Dialogue Scripts for {TargetName} to Directory"), FormatArguments);
		}

		// Prompt the user for the directory
		TArray<FString> SaveFilenames;
		if (DesktopPlatform->SaveFileDialog(ParentWindowWindowHandle, DialogTitle.ToString(), DefaultPath, DefaultFilename, FileTypes, 0, SaveFilenames))
		{
			LocalizationCommandletTasks::ExportDialogueScriptForCulture(ParentWindow.ToSharedRef(), LocalizationTarget, Culture->GetName(), TOptional<FString>(SaveFilenames.Top()));
		}
	}

	return FReply::Handled();
}

bool SLocalizationTargetEditorCultureRow::CanImportDialogue() const
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return Culture.IsValid() && LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationTargetEditorCultureRow::ImportDialogue()
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	if (Culture.IsValid() && LocalizationTarget)
	{
		// Warn about potentially loaded audio assets
		{
			TArray<ULocalizationTarget*> Targets;
			Targets.Add(LocalizationTarget);

			if (!LocalizationCommandletTasks::ReportLoadedAudioAssets(Targets))
			{
				return FReply::Handled();
			}
		}

		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		LocalizationCommandletTasks::ImportDialogueForCulture(ParentWindow.ToSharedRef(), LocalizationTarget, Culture->GetName());
	}

	return FReply::Handled();
}

bool SLocalizationTargetEditorCultureRow::CanCompileText() const
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	return Culture.IsValid() && LocalizationTarget && LocalizationTarget->Settings.SupportedCulturesStatistics.Num() > 0 && LocalizationTarget->Settings.SupportedCulturesStatistics.IsValidIndex(LocalizationTarget->Settings.NativeCultureIndex);
}

FReply SLocalizationTargetEditorCultureRow::CompileText()
{
	const FCulturePtr Culture = GetCulture();
	ULocalizationTarget* const LocalizationTarget = GetTarget();
	if (Culture.IsValid() && LocalizationTarget)
	{
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		LocalizationCommandletTasks::CompileTextForCulture(ParentWindow.ToSharedRef(), LocalizationTarget, Culture->GetName());
	}

	return FReply::Handled();
}

bool SLocalizationTargetEditorCultureRow::CanDelete() const
{
	return !IsNativeCultureForTarget();
}

FReply SLocalizationTargetEditorCultureRow::EnqueueDeletion()
{
	PropertyUtilities->EnqueueDeferredAction(FSimpleDelegate::CreateSP(this, &SLocalizationTargetEditorCultureRow::Delete));
	return FReply::Handled();
}

void SLocalizationTargetEditorCultureRow::Delete()
{
	static bool IsExecuting = false;
	if (!IsExecuting)
	{
		TGuardValue<bool> ReentranceGuard(IsExecuting, true);

		const FCulturePtr Culture = GetCulture();
		ULocalizationTarget* const LocalizationTarget = GetTarget();
		if (Culture.IsValid() && LocalizationTarget)
		{
			FText TitleText;
			FText MessageText;

			// Setup dialog for deleting supported culture.
			const FText FormatPattern = LOCTEXT("DeleteCultureConfirmationDialogMessage", "This action can not be undone and data will be permanently lost. Are you sure you would like to delete {CultureName}?");
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("CultureName"), FText::FromString(Culture->GetDisplayName()));
			MessageText = FText::Format(FormatPattern, Arguments);
			TitleText = LOCTEXT("DeleteCultureConfirmationDialogTitle", "Confirm Culture Deletion");

			switch(FMessageDialog::Open(EAppMsgType::OkCancel, MessageText, TitleText))
			{
			case EAppReturnType::Ok:
				{
					const FString CultureName = Culture->GetName();
					LocalizationTarget->DeleteFiles(&CultureName);

					// Remove this element from the parent array.
					const TSharedPtr<IPropertyHandle> SupportedCulturesStatisticsPropertyHandle = TargetSettingsPropertyHandle.IsValid() && TargetSettingsPropertyHandle->IsValidHandle() ? TargetSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLocalizationTargetSettings, SupportedCulturesStatistics)) : nullptr;
					if (SupportedCulturesStatisticsPropertyHandle.IsValid() && SupportedCulturesStatisticsPropertyHandle->IsValidHandle())
					{
						const TSharedPtr<IPropertyHandleArray> ArrayPropertyHandle = SupportedCulturesStatisticsPropertyHandle->AsArray();
						if (ArrayPropertyHandle.IsValid())
						{
							// Update the NativeCultureIndex if it needs to be changed based on the array resize
							// We do this before deleting the item so that SLocalizationTargetEditor::OnFinishedChangingProperties runs with the proper index
							FLocalizationTargetSettings* const TargetSettings = GetTargetSettings();
							if (CultureIndex < TargetSettings->NativeCultureIndex)
							{
								TargetSettingsPropertyHandle->NotifyPreChange();
								TargetSettings->NativeCultureIndex--;
								TargetSettingsPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
							}

							ArrayPropertyHandle->DeleteItem(CultureIndex);
						}
					}
				}
				break;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
