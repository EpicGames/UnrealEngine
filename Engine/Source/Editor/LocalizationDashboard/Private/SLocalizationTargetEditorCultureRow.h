// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "PropertyHandle.h"

class IPropertyUtilities;
class ULocalizationTarget;
struct FCultureStatistics;
struct FLocalizationTargetSettings;

class SLocalizationTargetEditorCultureRow : public SMultiColumnTableRow<FCulturePtr>
{
public:
	void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<IPropertyUtilities>& InPropertyUtilities, const TSharedRef<IPropertyHandle>& InTargetSettingsPropertyHandle, const int32 InCultureIndex);
	TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;

private:
	ULocalizationTarget* GetTarget() const;
	FLocalizationTargetSettings* GetTargetSettings() const;
	FCultureStatistics* GetCultureStatistics() const;
	FCulturePtr GetCulture() const;
	bool IsNativeCultureForTarget() const;

	void OnNativeCultureCheckStateChanged(const ECheckBoxState CheckState);
	uint32 GetWordCount() const;
	uint32 GetNativeWordCount() const;

	FText GetCultureDisplayName() const;
	FText GetCultureName() const;

	FText GetWordCountText() const;
	TOptional<float> GetProgressPercentage() const;

	void UpdateTargetFromReports();

	bool CanEditText() const;
	FReply EditText();

	bool CanImportText() const;
	FReply ImportText();

	bool CanExportText() const;
	FReply ExportText();

	bool CanImportDialogueScript() const;
	FReply ImportDialogueScript();

	bool CanExportDialogueScript() const;
	FReply ExportDialogueScript();

	bool CanImportDialogue() const;
	FReply ImportDialogue();

	bool CanCompileText() const;
	FReply CompileText();

	bool CanDelete() const;
	FReply EnqueueDeletion();
	void Delete();

private:
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
	TSharedPtr<IPropertyHandle> TargetSettingsPropertyHandle;
	int32 CultureIndex;
};
