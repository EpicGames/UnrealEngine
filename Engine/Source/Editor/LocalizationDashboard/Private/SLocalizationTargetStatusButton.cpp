// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLocalizationTargetStatusButton.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "LocalizationConfigurationScript.h"
#include "LocalizationTargetTypes.h"
#include "Math/Color.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SCompoundWidget.h"

struct FSlateBrush;

#define LOCTEXT_NAMESPACE "LocalizationTargetStatusButton"

void SLocalizationTargetStatusButton::Construct(const FArguments& InArgs, ULocalizationTarget& InTarget)
{
	Target = &InTarget;

	SButton::Construct(
		SButton::FArguments()
		.ButtonStyle( FAppStyle::Get(), TEXT("HoverHintOnly") )
		.OnClicked(this, &SLocalizationTargetStatusButton::OnClicked)
		);

		ChildSlot
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(this, &SLocalizationTargetStatusButton::GetImageBrush)
				.ColorAndOpacity(this, &SLocalizationTargetStatusButton::GetColorAndOpacity)
				.ToolTipText(this, &SLocalizationTargetStatusButton::GetToolTipText)
			];
}

const FSlateBrush* SLocalizationTargetStatusButton::GetImageBrush() const
{
	switch (Target->Settings.ConflictStatus)
	{
	default:
	case ELocalizationTargetConflictStatus::Unknown:
		return FAppStyle::GetBrush("Icons.Warning");
	case ELocalizationTargetConflictStatus::Clear:
		return FAppStyle::GetBrush("Symbols.Check");
	case ELocalizationTargetConflictStatus::ConflictsPresent:
		return FAppStyle::GetBrush("Symbols.X");
	}
}

FSlateColor SLocalizationTargetStatusButton::GetColorAndOpacity() const
{
	switch (Target->Settings.ConflictStatus)
	{
	default:
	case ELocalizationTargetConflictStatus::Unknown:
		return FLinearColor::White;
	case ELocalizationTargetConflictStatus::Clear:
		return FLinearColor::Green;
	case ELocalizationTargetConflictStatus::ConflictsPresent:
		return FLinearColor::Red;
	}
}

FText SLocalizationTargetStatusButton::GetToolTipText() const
{
	switch (Target->Settings.ConflictStatus)
	{
	default:
	case ELocalizationTargetConflictStatus::Unknown:
		return LOCTEXT("StatusToolTip_Unknown", "Conflict report file not detected. Perform a gather to generate a conflict report file.");
	case ELocalizationTargetConflictStatus::Clear:
		return LOCTEXT("StatusToolTip_Clear", "No conflicts detected.");
	case ELocalizationTargetConflictStatus::ConflictsPresent:
		return LOCTEXT("StatusToolTip_ConflictsPresent", "Conflicts detected. Click to open the conflict report.");
	}
}

FReply SLocalizationTargetStatusButton::OnClicked()
{
	switch (Target->Settings.ConflictStatus)
	{
	case ELocalizationTargetConflictStatus::Unknown:
		// Do nothing.
		break;
	case ELocalizationTargetConflictStatus::Clear:
		// Do nothing.
		break;
	case ELocalizationTargetConflictStatus::ConflictsPresent:
		FPlatformProcess::LaunchFileInDefaultExternalApplication(*FPaths::ConvertRelativePathToFull(LocalizationConfigurationScript::GetConflictReportPath(Target)), nullptr, ELaunchVerb::Open);
		break;
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
