// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"

class ULocalizationTarget;
struct FSlateBrush;

class SLocalizationTargetStatusButton : public SButton
{
public:
	SLATE_BEGIN_ARGS( SLocalizationTargetStatusButton ){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ULocalizationTarget& Target);

private:
	const FSlateBrush* GetImageBrush() const;
	FSlateColor GetColorAndOpacity() const;
	FText GetToolTipText() const;
	FReply OnClicked();

private:
	ULocalizationTarget* Target;
};
