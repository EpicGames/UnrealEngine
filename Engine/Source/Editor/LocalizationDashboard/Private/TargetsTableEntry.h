// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "LocalizationTargetTypes.h"
#include "PropertyHandle.h"

struct FTargetsTableEntry
{
	TSharedPtr<IPropertyHandle> TargetObjectPropertyHandle;
	int32 CultureIndex;

	FTargetsTableEntry()
		: CultureIndex(INDEX_NONE)
	{}

	FTargetsTableEntry(const TSharedPtr<IPropertyHandle>& InTargetObjectPropertyHandle, const int32 InCultureIndex = INDEX_NONE)
		: TargetObjectPropertyHandle(InTargetObjectPropertyHandle)
		, CultureIndex(InCultureIndex)
	{}

	ULocalizationTarget* GetTarget()
	{
		if (TargetObjectPropertyHandle.IsValid() && TargetObjectPropertyHandle->IsValidHandle())
		{
			UObject* Value = nullptr;
			TargetObjectPropertyHandle->GetValue(Value);
			return Cast<ULocalizationTarget>(Value);
		}
		return nullptr;
	}
};