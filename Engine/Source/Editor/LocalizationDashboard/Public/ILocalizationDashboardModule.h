// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class ILocalizationServiceProvider;

/**
 * Interface for localization dashboard module.
 */
class ILocalizationDashboardModule
	: public IModuleInterface
{
public:
	/** Virtual destructor. */
	virtual ~ILocalizationDashboardModule() = default;

	/**
	 * Shows the localization dashboard UI.
	 */
	virtual void Show() = 0;

	virtual const TArray<ILocalizationServiceProvider*>& GetLocalizationServiceProviders() const = 0;
	virtual ILocalizationServiceProvider* GetCurrentLocalizationServiceProvider() const = 0;

	static ILocalizationDashboardModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ILocalizationDashboardModule>("LocalizationDashboard");
	}
};
